{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert - Main",
	"Description": "Allow collaborative multi-users sessions in the Editor",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "Concert",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ]
		},
		{
			"Name": "ConcertClient",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramDenyList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ]
		},
		{
			"Name": "ConcertServer",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ]
		},
		{
			"Name": "ConcertTransport",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor", "LiveLinkHub" ]
		}
	]
}
