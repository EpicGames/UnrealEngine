// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogger.h"
#include "ConcertMessages.h"
#include "HAL/IConsoleManager.h"
#include "IConcertMessages.h"
#include "ConcertLogGlobal.h"
#include "ConcertTransportEvents.h"
#include "ConcertUtil.h"
#include "Algo/AllOf.h"

#include "Logging/LogVerbosity.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "HAL/FileManager.h"
#include "Containers/Ticker.h"
#include "Templates/SharedPointerInternals.h"
#include "UObject/UnrealType.h"
#include "UObject/StructOnScope.h"
#include "UObject/PropertyPortFlags.h"


namespace ConcertLoggerUtil
{

FString PayloadToString(const FStructOnScope& InPayload)
{
	FString Result;
	if (InPayload.GetStruct() && InPayload.GetStructMemory())
	{
		check(InPayload.GetStruct()->IsA<UScriptStruct>());
		UScriptStruct* InStruct = (UScriptStruct*)InPayload.GetStruct();
		InStruct->ExportText(Result, InPayload.GetStructMemory(), InPayload.GetStructMemory(), nullptr, PPF_None, nullptr);
	}
	return Result;
}

FString SerializedPayloadToString(const FConcertSessionSerializedPayload& InPayload)
{
	FStructOnScope TempPayload;
	if (InPayload.PayloadSize < 512 && InPayload.GetPayload(TempPayload))
	{
		return PayloadToString(TempPayload);
	}
	else if (InPayload.PayloadSize >= 512)
	{
		return TEXT("Payload string disabled. Payload is too large!");
	}

	return FString();
}

void PopulateLogMessagePayload(const FConcertSessionSerializedPayload& InPayload, FConcertLog& InOutLogMessage)
{
	InOutLogMessage.CustomPayloadTypename = InPayload.PayloadTypeName;
	InOutLogMessage.CustomPayloadUncompressedByteSize = InPayload.PayloadSize;
	InOutLogMessage.StringPayload = SerializedPayloadToString(InPayload);
}

void PopulateLogMessage(const UScriptStruct* InMessageType, const void* InMessageData, const FGuid& InSourceEndpoint, const FGuid& InDestinationEndpoint, const FDateTime& InUtcNow, const EConcertLogMessageAction InMessageAction, FConcertLog& InOutLogMessage)
{
	checkf(InMessageType->IsChildOf(FConcertMessageData::StaticStruct()), TEXT("PopulateLogMessage can only be used with messages deriving from FConcertMessageData!"));

	const FConcertMessageData* ConcertMessageData = (const FConcertMessageData*)InMessageData;

	InOutLogMessage.Frame = GFrameCounter;
	InOutLogMessage.MessageId = ConcertMessageData->MessageId;
	InOutLogMessage.MessageOrderIndex = ConcertMessageData->MessageOrderIndex;
	InOutLogMessage.ChannelId = ConcertMessageData->ChannelId;
	InOutLogMessage.Timestamp = InUtcNow;
	InOutLogMessage.MessageAction = InMessageAction;
	InOutLogMessage.MessageTypeName = InMessageType->GetFName();
	InOutLogMessage.OriginEndpointId = InSourceEndpoint;
	InOutLogMessage.DestinationEndpointId = InDestinationEndpoint;
	InOutLogMessage.CustomPayloadUncompressedByteSize = 0;

	if (InMessageType->IsChildOf(FConcertSession_CustomEvent::StaticStruct()))
	{
		const FConcertSession_CustomEvent* ConcertCustomEventData = (const FConcertSession_CustomEvent*)InMessageData;
		InOutLogMessage.SerializedPayload = ConcertCustomEventData->SerializedPayload;
	}
	else if (InMessageType->IsChildOf(FConcertSession_CustomRequest::StaticStruct()))
	{
		const FConcertSession_CustomRequest* ConcertCustomRequestData = (const FConcertSession_CustomRequest*)InMessageData;
		InOutLogMessage.SerializedPayload = ConcertCustomRequestData->SerializedPayload;
	}
	else if (InMessageType->IsChildOf(FConcertSession_CustomResponse::StaticStruct()))
	{
		const FConcertSession_CustomResponse* ConcertCustomResponseData = (const FConcertSession_CustomResponse*)InMessageData;
		InOutLogMessage.SerializedPayload = ConcertCustomResponseData->SerializedPayload;
	}
}

FConcertLog BuildLogMessage(const UScriptStruct* InMessageType, const void* InMessageData, const FGuid& InSourceEndpoint, const FGuid& InDestinationEndpoint, const FDateTime& InUtcNow, const EConcertLogMessageAction InMessageAction)
{
	FConcertLog LogMessage;
	PopulateLogMessage(InMessageType, InMessageData, InSourceEndpoint, InDestinationEndpoint, InUtcNow, InMessageAction, LogMessage);
	return LogMessage;
}

FString MessageTypeToString(const UScriptStruct* InMessageType, const void* InMessageData)
{
	FString MessageTypeStr = InMessageType->GetName();

	if (InMessageData)
	{
		if (InMessageType->IsChildOf(FConcertSession_CustomEvent::StaticStruct()))
		{
			const FConcertSession_CustomEvent* ConcertCustomEventData = (const FConcertSession_CustomEvent*)InMessageData;
			MessageTypeStr += FString::Printf(TEXT("(%s)"), *ConcertCustomEventData->SerializedPayload.PayloadTypeName.ToString());
		}
		else if (InMessageType->IsChildOf(FConcertSession_CustomRequest::StaticStruct()))
		{
			const FConcertSession_CustomRequest* ConcertCustomRequestData = (const FConcertSession_CustomRequest*)InMessageData;
			MessageTypeStr += FString::Printf(TEXT("(%s)"), *ConcertCustomRequestData->SerializedPayload.PayloadTypeName.ToString());
		}
		else if (InMessageType->IsChildOf(FConcertSession_CustomResponse::StaticStruct()))
		{
			const FConcertSession_CustomResponse* ConcertCustomResponseData = (const FConcertSession_CustomResponse*)InMessageData;
			MessageTypeStr += FString::Printf(TEXT("(%s)"), *ConcertCustomResponseData->SerializedPayload.PayloadTypeName.ToString());
		}
	}

	return MessageTypeStr;
}

FString GetMessageTypeString(const TSharedRef<IConcertMessage>& Message)
{
	return MessageTypeToString(Message->GetMessageType(), Message->GetMessageTemplate());
}

FString GetMessageTypeString(const FConcertMessageContext& ConcertContext)
{
	return MessageTypeToString(ConcertContext.MessageType, ConcertContext.Message);
}

FString MessageToString(const FGuid& MessageId, const UScriptStruct* InMessageType, const void* InMessageData)
{
	return FString::Printf(TEXT("'%s' (%s)"), *MessageId.ToString(), *MessageTypeToString(InMessageType, InMessageData));
}

FString GetMessageString(const TSharedRef<IConcertMessage>& Message)
{
	return MessageToString(Message->GetMessageId(), Message->GetMessageType(), Message->GetMessageTemplate());
}

FString GetMessageString(const FConcertMessageContext& ConcertContext)
{
	return MessageToString(ConcertContext.Message->MessageId, ConcertContext.MessageType, ConcertContext.Message);
}

const TCHAR* ReliableHandshakeStateToString(const EConcertReliableHandshakeState InState)
{
	switch (InState)
	{
	case EConcertReliableHandshakeState::Negotiate:
		return TEXT("negotiate");
	case EConcertReliableHandshakeState::Success:
		return TEXT("success");
	default:
		return TEXT("");
	}
}

}

struct FActiveLoggers
{
	FActiveLoggers() :
		 EnableLoggingCommand(TEXT("Concert.EnableLogging"), TEXT("Enable Additional Logging for Concert."),
							  FConsoleCommandDelegate::CreateRaw(this, &FActiveLoggers::EnableLogging)),
		 DisableLoggingCommand(TEXT("Concert.DisableLogging"), TEXT("Disable Additional Logging for Concert."),
							   FConsoleCommandDelegate::CreateRaw(this, &FActiveLoggers::DisableLogging)),
		 ChangeConcertVerbosityCommand(TEXT("Concert.SetLoggingLevel"), TEXT("Change the logging level for Concert loggers."),
									   FConsoleCommandWithArgsDelegate::CreateRaw(this, &FActiveLoggers::ChangeVerbosity))
	{
	}

	/** Enable additional concert logging. */
	void EnableLogging()
	{
		for (TWeakPtr<FConcertLogger, ESPMode::ThreadSafe>& Logger : Loggers)
		{
			if (TSharedPtr<FConcertLogger, ESPMode::ThreadSafe> LoggerPtr = Logger.Pin())
			{
				if (!LoggerPtr->IsLogging())
				{
					LoggerPtr->StartLogging();
				}
			}
		}

		ConcertTransportEvents::OnConcertTransportLoggingEnabledChangedEvent().Broadcast(true);
	}

	/** Disable additional concert logging. */
	void DisableLogging()
	{
		for (TWeakPtr<FConcertLogger, ESPMode::ThreadSafe>& Logger : Loggers)
		{
			if (TSharedPtr<FConcertLogger, ESPMode::ThreadSafe> LoggerPtr = Logger.Pin())
			{
				if (LoggerPtr->IsLogging())
				{
					LoggerPtr->StopLogging();
				}
			}
		}
		
		ConcertTransportEvents::OnConcertTransportLoggingEnabledChangedEvent().Broadcast(false);
	}

	bool AreAllLoggersEnabled() const
	{
		return Algo::AllOf(Loggers, [](const TWeakPtr<FConcertLogger, ESPMode::ThreadSafe>& Logger)
		{
			TSharedPtr<FConcertLogger, ESPMode::ThreadSafe> LoggerPtr = Logger.Pin();
			return LoggerPtr.IsValid() && LoggerPtr->IsLogging();
		});
	}

	void ChangeVerbosity(const TArray<FString>& Args)
	{
		if ( Args.Num() < 1 )
		{
			UE_LOGF(LogConcert, Log, "Usage: Concert.SetLoggingLevel VerbosityLevel");
			return;
		}
#if !NO_LOGGING
		if ( Args[0] == TEXT("VeryVerbose") )
		{
			LogConcert.SetVerbosity(ELogVerbosity::VeryVerbose);
			LogConcertDebug.SetVerbosity(ELogVerbosity::VeryVerbose);
		}
		else if (Args[0] == TEXT("Verbose") )
		{
			LogConcert.SetVerbosity(ELogVerbosity::Verbose);
			LogConcertDebug.SetVerbosity(ELogVerbosity::Verbose);
		}
		else if (Args[0] == TEXT("Default") )
		{
			LogConcert.SetVerbosity(ELogVerbosity::Log);
			LogConcertDebug.SetVerbosity(ELogVerbosity::Log);
		}
#endif
	}

	/** Console command to turn on logger. */
	FAutoConsoleCommand EnableLoggingCommand;

	/** Console command to turn off logger. */
	FAutoConsoleCommand DisableLoggingCommand;

	/** Console command to change concert verbosity. */
	FAutoConsoleCommand ChangeConcertVerbosityCommand;

	/** List of active loggers */
	TArray<TWeakPtr<FConcertLogger,ESPMode::ThreadSafe>> Loggers;
};

FActiveLoggers& GetActiveLoggers()
{
	static FActiveLoggers ActiveLoggers;
	return ActiveLoggers;
}

namespace ConcertTransportEvents
{
	bool IsLoggingEnabled()
	{
		return GetActiveLoggers().AreAllLoggersEnabled();
	}
	
	void SetLoggingEnabled(bool bEnabled)
	{
		if (bEnabled)
		{
			 GetActiveLoggers().EnableLogging();
		}
		else
		{
			GetActiveLoggers().DisableLogging();
		}
	}
}

IConcertTransportLoggerRef FConcertLogger::CreateLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc)
{
	FActiveLoggers& ActiveLoggers = GetActiveLoggers();
	TSharedPtr<FConcertLogger, ESPMode::ThreadSafe> SharedLogger = MakeShared<FConcertLogger, ESPMode::ThreadSafe>(InOwnerContext, MoveTemp(LogListenerFunc));
	ActiveLoggers.Loggers.Add(SharedLogger);
	return SharedLogger.ToSharedRef();
}


void FConcertLogger::SetVerboseLogging(bool bInState)
{
	FActiveLoggers& ActiveLoggers = GetActiveLoggers();
	if (bInState)
	{
		ActiveLoggers.EnableLogging();
		ActiveLoggers.ChangeVerbosity({TEXT("VeryVerbose")});
	}
	else
	{
		ActiveLoggers.DisableLogging();
		ActiveLoggers.ChangeVerbosity({TEXT("Default")});
	}
}

FConcertLogger::FConcertLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc)
	: bIsLogging(false)
	, OwnerContext(InOwnerContext)
	, LogListenerFunc(MoveTemp(LogListenerFunc))
{}

FConcertLogger::~FConcertLogger()
{
	InternalStopLogging();
}

bool FConcertLogger::IsLogging() const
{
	return bIsLogging;
}

void FConcertLogger::StartLogging()
{
	FScopeLock CSVArchiveLock(&CSVArchiveCS);
	InternalStartLogging();
}

void FConcertLogger::StopLogging()
{
	FScopeLock CSVArchiveLock(&CSVArchiveCS);
	InternalStopLogging();
}

void FConcertLogger::FlushLog()
{
	FScopeLock CSVArchiveLock(&CSVArchiveCS);
	InternalFlushLog();
}

void FConcertLogger::LogTimeOut(const TSharedRef<IConcertMessage>& Message, const FGuid& EndpointId, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, Warning, "%ls: Message %ls timed-out sending to '%ls'.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(Message), *EndpointId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(Message->GetMessageType(), Message->GetMessageTemplate(), Message->GetSenderId(), EndpointId, UtcNow, EConcertLogMessageAction::TimeOut));
}

void FConcertLogger::LogSendAck(const FConcertAckData& AckData, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Acknowledgement '%ls' sent to '%ls' for message '%ls'.", *OwnerContext.ToString(), *AckData.MessageId.ToString(), *DestEndpoint.ToString(), *AckData.SourceMessageId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			AckData.MessageId,
			AckData.MessageOrderIndex,
			AckData.ChannelId,
			FDateTime(AckData.AckSendTimeTicks),
			EConcertLogMessageAction::Send,
			FConcertAckData::StaticStruct()->GetFName(),
			AckData.ConcertEndpointId,
			DestEndpoint,
			NAME_None,
			0
		}
	);
}

void FConcertLogger::LogSendEndpointClosed(const FConcertEndpointClosedData& EndpointClosedData, const FGuid& DestEndpoint, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Endpoint closure '%ls' sent to '%ls'.", *OwnerContext.ToString(), *EndpointClosedData.MessageId.ToString(), *DestEndpoint.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			EndpointClosedData.MessageId,
			EndpointClosedData.MessageOrderIndex,
			EndpointClosedData.ChannelId,
			UtcNow,
			EConcertLogMessageAction::Send,
			FConcertEndpointClosedData::StaticStruct()->GetFName(),
			EndpointClosedData.ConcertEndpointId,
			DestEndpoint,
			NAME_None,
			0
		}
	);
}

void FConcertLogger::LogSendReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Handshake sent to '%ls' (state: %ls, channel: %d, index: %d).", 
		*OwnerContext.ToString(),
		*DestEndpoint.ToString(), 
		ConcertLoggerUtil::ReliableHandshakeStateToString(ReliableHandshakeData.HandshakeState),
		ReliableHandshakeData.ReliableChannelId, 
		ReliableHandshakeData.NextMessageIndex
	);

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			ReliableHandshakeData.MessageId,
			ReliableHandshakeData.MessageOrderIndex,
			ReliableHandshakeData.ChannelId,
			UtcNow,
			EConcertLogMessageAction::Send,
			FConcertReliableHandshakeData::StaticStruct()->GetFName(),
			ReliableHandshakeData.ConcertEndpointId,
			DestEndpoint,
			NAME_None,
			0
		}
	);
}

void FConcertLogger::LogReceiveReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Handshake received from '%ls' (state: %ls, channel: %d, index: %d).", 
		*OwnerContext.ToString(),
		*ReliableHandshakeData.ConcertEndpointId.ToString(), 
		ConcertLoggerUtil::ReliableHandshakeStateToString(ReliableHandshakeData.HandshakeState),
		ReliableHandshakeData.ReliableChannelId, 
		ReliableHandshakeData.NextMessageIndex
	);

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			ReliableHandshakeData.MessageId,
			ReliableHandshakeData.MessageOrderIndex,
			ReliableHandshakeData.ChannelId,
			UtcNow,
			EConcertLogMessageAction::Receive,
			FConcertReliableHandshakeData::StaticStruct()->GetFName(),
			ReliableHandshakeData.ConcertEndpointId,
			DestEndpoint,
			NAME_None,
			0
		}
	);
}

void FConcertLogger::LogPublish(const TSharedRef<IConcertMessage>& Message)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls published.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(Message));

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(Message->GetMessageType(), Message->GetMessageTemplate(), Message->GetSenderId(), FGuid(), Message->GetCreationDate(), EConcertLogMessageAction::Publish));
}

void FConcertLogger::LogSend(const TSharedRef<IConcertMessage>& Message, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls sent to '%ls'.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(Message), *DestEndpoint.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(Message->GetMessageType(), Message->GetMessageTemplate(), Message->GetSenderId(), DestEndpoint, Message->GetCreationDate(), EConcertLogMessageAction::Send));
}

void FConcertLogger::LogMessageReceived(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls received from '%ls'.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext), *ConcertContext.SenderConcertEndpointId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Receive));
}

void FConcertLogger::LogMessageQueued(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls queued.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext));

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Queue));
}

void FConcertLogger::LogMessageDiscarded(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint, const EMessageDiscardedReason Reason)
{
	switch (Reason)
	{
	case EMessageDiscardedReason::NotRequired:
		UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls discarded. Message was not required.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext));
		break;

	case EMessageDiscardedReason::AlreadyProcessed:
		UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Message %ls discarded. Message was already processed.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext));
		break;

	case EMessageDiscardedReason::UnknownEndpoint:
		UE_LOGF(LogConcertDebug, Warning, "%ls: Message %ls discarded. Unknown remote endpoint '%ls'.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext), *ConcertContext.Message->ConcertEndpointId.ToString());
		break;

	default:
		checkf(false, TEXT("Unknown EMessageDiscardedReason!"));
		break;
	}

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, Reason == EMessageDiscardedReason::AlreadyProcessed ? EConcertLogMessageAction::Duplicate : EConcertLogMessageAction::Discard));
}

void FConcertLogger::LogProcessEvent(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Event %ls processed.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext));

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Process));
}

void FConcertLogger::LogProcessRequest(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Request %ls processed.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext));

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Process));
}

void FConcertLogger::LogProcessResponse(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) 
{
	const FConcertResponseData* ResponseMessage = ConcertContext.GetMessage<FConcertResponseData>();
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Response %ls processed for request '%ls'.", *OwnerContext.ToString(), *ConcertLoggerUtil::GetMessageString(ConcertContext), *ResponseMessage->RequestMessageId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Process));
}

void FConcertLogger::LogProcessAck(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	const FConcertAckData* AckMessage = ConcertContext.GetMessage<FConcertAckData>();
	UE_LOGF(LogConcertDebug, VeryVerbose, "%ls: Acknowledgement '%ls' processed for message '%ls'.", *OwnerContext.ToString(), *AckMessage->MessageId.ToString(), *AckMessage->SourceMessageId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::Process));
}

void FConcertLogger::LogRemoteEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint)
{
	UE_LOGF(LogConcertDebug, Display, "%ls: Remote endpoint '%ls' discovered.", *OwnerContext.ToString(), *ConcertContext.SenderConcertEndpointId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(ConcertLoggerUtil::BuildLogMessage(ConcertContext.MessageType, ConcertContext.Message, ConcertContext.SenderConcertEndpointId, DestEndpoint, ConcertContext.UtcNow, EConcertLogMessageAction::EndpointDiscovery));
}

void FConcertLogger::LogRemoteEndpointTimeOut(const FGuid& EndpointId, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, Display, "%ls: Remote endpoint '%ls' timed-out.", *OwnerContext.ToString(), *EndpointId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			FGuid(),
			0,
			FConcertMessageData::UnreliableChannelId,
			UtcNow,
			EConcertLogMessageAction::EndpointTimeOut,
			NAME_None,
			EndpointId,
			FGuid(),
			NAME_None,
			0
		}
	);
}

void FConcertLogger::LogRemoteEndpointClosure(const FGuid& EndpointId, const FDateTime& UtcNow)
{
	UE_LOGF(LogConcertDebug, Display, "%ls: Remote endpoint '%ls' closed by remote peer.", *OwnerContext.ToString(), *EndpointId.ToString());

	if (!IsLogging())
	{
		return;
	}

	LogQueue.Enqueue(FConcertLog
		{
			GFrameCounter,
			FGuid(),
			0,
			FConcertMessageData::UnreliableChannelId,
			UtcNow,
			EConcertLogMessageAction::EndpointClosure,
			NAME_None,
			EndpointId,
			FGuid(),
			NAME_None,
			0
		}
	);
}

void FConcertLogger::InternalStartLogging()
{
	const FString CSVFilename = FPaths::ProjectLogDir() / TEXT("Concert") / FString::Printf(TEXT("%s-%s-%s.csv"), FApp::GetProjectName(), *OwnerContext.EndpointFriendlyName, *FDateTime::Now().ToString());
	CSVArchive = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*CSVFilename, EFileWrite::FILEWRITE_AllowRead));
	if (CSVArchive)
	{
		UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
		CSVArchive->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));
	}
	LogHeader();
	bIsLogging = CSVArchive.IsValid();
}

void FConcertLogger::InternalStopLogging()
{
	InternalFlushLog();

	bIsLogging = false;
	CSVArchive.Reset();
}

void FConcertLogger::InternalFlushLog()
{
	if (!IsLogging())
	{
		return;
	}

	if (UE::IsSavingPackage() || IsGarbageCollecting())
	{
		// Cannot process payload data when this is happening
		return;
	}

	// Process logs
	FConcertLog Log;
	while (LogQueue.Dequeue(Log))
	{
		LogEntry(Log);
	}
	if (CSVArchive.IsValid())
	{
		CSVArchive->Flush();
	}
}

void FConcertLogger::LogHeader()
{
	FString CSVHeader;
	for (TFieldIterator<const FProperty> PropertyIt(FConcertLog::StaticStruct(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); PropertyIt; ++PropertyIt)
	{
		if (PropertyIt->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		if (!CSVHeader.IsEmpty())
		{
			CSVHeader += TEXT(",");
		}

		FString PropertyName = PropertyIt->GetName();
		PropertyName.ReplaceInline(TEXT("\""), TEXT("\"\""));

		CSVHeader += TEXT("\"");
		CSVHeader += PropertyName;
		CSVHeader += TEXT("\"");
	}
	CSVHeader += LINE_TERMINATOR;

	if (CSVArchive.IsValid())
	{
		FTCHARToUTF8 CSVHeaderUTF8(*CSVHeader);
		CSVArchive->Serialize((UTF8CHAR*)CSVHeaderUTF8.Get(), CSVHeaderUTF8.Length() * sizeof(UTF8CHAR));
	}
}

void FConcertLogger::LogEntry(FConcertLog& Log)
{
	SCOPED_CONCERT_TRACE(FConcertLogger_LogEntry);
	static const FName MessageOrderIndexPropertyName = TEXT("MessageOrderIndex");
	const bool bIsReliable = Log.ChannelId != FConcertMessageData::UnreliableChannelId;

	// Process the log payload into its string form (we do this now as it may not be safe to do it when the logging actually happens)
	if (!Log.SerializedPayload.PayloadTypeName.IsNone())
	{
		ConcertLoggerUtil::PopulateLogMessagePayload(Log.SerializedPayload, Log);
	}

	// Inform external systems about the log
	LogListenerFunc(Log);

	FString CSVRow;
	for (TFieldIterator<const FProperty> PropertyIt(FConcertLog::StaticStruct(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); PropertyIt; ++PropertyIt)
	{
		if (PropertyIt->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		if (!CSVRow.IsEmpty())
		{
			CSVRow += TEXT(",");
		}

		// Skip exporting the MessageOrderIndex value for unreliable messages as they add a lot of noise to the log
		const bool bExportValue = bIsReliable || PropertyIt->GetFName() != MessageOrderIndexPropertyName;

		FString PropertyValue;
		if (bExportValue)
		{
			PropertyIt->ExportTextItem_InContainer(PropertyValue, &Log, nullptr, nullptr, PPF_None);
		}
		PropertyValue.ReplaceInline(TEXT("\""), TEXT("\"\""));

		CSVRow += TEXT("\"");
		CSVRow += PropertyValue;
		CSVRow += TEXT("\"");
	}
	CSVRow += LINE_TERMINATOR;

	if (CSVArchive.IsValid())
	{
		FTCHARToUTF8 CSVRowUTF8(*CSVRow);
		CSVArchive->Serialize((UTF8CHAR*)CSVRowUTF8.Get(), CSVRowUTF8.Length() * sizeof(UTF8CHAR));
	}
}
