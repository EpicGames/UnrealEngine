// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "IConcertEndpoint.h"
#include "ConcertMessageData.h"
#include "IConcertTransportLogger.h"
#include "ConcertUtil.h"
struct FConcertLog;
struct FConcertMessageContext;

class FConcertLogger : public IConcertTransportLogger
{
public:

	/** Factory function for use with FConcertTransportLoggerFactory */
	static IConcertTransportLoggerRef CreateLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc);

	/** Static function to enable / disable verbose logging. */
	static void SetVerboseLogging(bool bInState);

	FConcertLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc);
	virtual ~FConcertLogger() override;

	// IConcertTransportLogger interface
	virtual bool IsLogging() const override;
	virtual void StartLogging() override;
	virtual void StopLogging() override;
	virtual void FlushLog() override;
	virtual void LogTimeOut(const TSharedRef<IConcertMessage>& Message, const FGuid& EndpointId, const FDateTime& UtcNow) override;
	virtual void LogSendAck(const FConcertAckData& AckData, const FGuid& DestEndpoint) override;
	virtual void LogSendEndpointClosed(const FConcertEndpointClosedData& EndpointClosedData, const FGuid& DestEndpoint, const FDateTime& UtcNow) override;
	virtual void LogSendReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow) override;
	virtual void LogReceiveReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow) override;
	virtual void LogPublish(const TSharedRef<IConcertMessage>& Message) override;
	virtual void LogSend(const TSharedRef<IConcertMessage>& Message, const FGuid& DestEndpoint) override;
	virtual void LogMessageReceived(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogMessageQueued(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogMessageDiscarded(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint, const EMessageDiscardedReason Reason) override;
	virtual void LogProcessEvent(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogProcessRequest(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogProcessResponse(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogProcessAck(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogRemoteEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) override;
	virtual void LogRemoteEndpointTimeOut(const FGuid& EndpointId, const FDateTime& UtcNow) override;
	virtual void LogRemoteEndpointClosure(const FGuid& EndpointId, const FDateTime& UtcNow) override;

private:
	void InternalStartLogging();
	void InternalStopLogging();
	void InternalFlushLog();

	void LogHeader();
	void LogEntry(FConcertLog& Log);

	/** */
	bool bIsLogging;
	
	/** */
	FConcertEndpointContext OwnerContext;

	/** Called after a log is processed */
	FLogListener LogListenerFunc;

	/** Queue for unprocessed logs */
	TQueue<FConcertLog, EQueueMode::Mpsc> LogQueue;

	/** Archive & CS used to write CSV file, if any */
	mutable FCriticalSection CSVArchiveCS;
	TUniquePtr<FArchive> CSVArchive;
};
