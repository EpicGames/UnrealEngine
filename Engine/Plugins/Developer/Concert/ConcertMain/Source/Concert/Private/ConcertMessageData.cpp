// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertMessageData.h"
#include "ConcertLogGlobal.h"
#include "HAL/IConsoleManager.h"
#include "IdentifierTable/ConcertTransportArchives.h"

#include "Misc/App.h"
#include "Misc/Compression.h"
#include "UObject/StructOnScope.h"

#include "StructSerializer.h"
#include "StructDeserializer.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Backends/CborStructDeserializerBackend.h"

#include "Math/UnitConversion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertMessageData)

LLM_DEFINE_TAG(Concert_ConcertClientInfo);

void FConcertInstanceInfo::Initialize()
{
	InstanceId = FApp::GetInstanceId();
	InstanceName = FApp::GetInstanceName();

	if (IsRunningDedicatedServer())
	{
		InstanceType = TEXT("Server");
	}
	else if (FApp::IsGame())
	{
		InstanceType = TEXT("Game");
	}
	else if (IsRunningCommandlet())
	{
		InstanceType = TEXT("Commandlet");
	}
	else if (GIsEditor)
	{
		InstanceType = TEXT("Editor");
	}
	else
	{
		InstanceType = TEXT("Other");
	}
}

FText FConcertInstanceInfo::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertInstanceInfo", "InstanceName", "Instance Name: {0}"), FText::FromString(InstanceName));
	return TextBuilder.ToText();
}

bool FConcertInstanceInfo::operator==(const FConcertInstanceInfo& Other) const
{
	return	InstanceId == Other.InstanceId &&
			InstanceName == Other.InstanceName &&
			InstanceType == Other.InstanceType;
}

bool FConcertInstanceInfo::operator!=(const FConcertInstanceInfo& Other) const
{
	return !operator==(Other);
}

void FConcertServerInfo::Initialize()
{
	ServerName = FPlatformProcess::ComputerName();
	InstanceInfo.Initialize();
	InstanceInfo.InstanceType = TEXT("Server");
	ServerFlags = EConcertServerFlags::None;
}

FText FConcertServerInfo::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertServerInfo", "ServerName", "Server Name: {0}"), FText::FromString(ServerName));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertServerInfo", "AdminEndpointId", "Admin Endpoint ID: {0}"), FText::FromString(AdminEndpointId.ToString()));
	TextBuilder.AppendLine(InstanceInfo.ToDisplayString());
	return TextBuilder.ToText();
}

void FConcertClientInfo::Initialize()
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientInfo);
	InstanceInfo.Initialize();
	DeviceName = FPlatformProcess::ComputerName();
	PlatformName = FPlatformProperties::PlatformName();
	UserName = FApp::GetSessionOwner();
	bHasEditorData = WITH_EDITORONLY_DATA;
	bRequiresCookedData = FPlatformProperties::RequiresCookedData();
}

FText FConcertClientInfo::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertClientInfo", "DeviceName", "Device Name: {0}"), FText::FromString(DeviceName));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertClientInfo", "PlatformName", "Platform Name: {0}"), FText::FromString(PlatformName));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertClientInfo", "UserName", "User Name: {0}"), FText::FromString(UserName));
	TextBuilder.AppendLine(InstanceInfo.ToDisplayString());
	return TextBuilder.ToText();
}

bool FConcertClientInfo::operator==(const FConcertClientInfo& Other) const
{
	return	InstanceInfo == Other.InstanceInfo &&
			DeviceName == Other.DeviceName &&
			PlatformName == Other.PlatformName &&
			UserName == Other.UserName &&
			DisplayName == Other.DisplayName &&
			AvatarColor == Other.AvatarColor &&
			DesktopAvatarActorClass == Other.DesktopAvatarActorClass &&
			VRAvatarActorClass == Other.VRAvatarActorClass &&
			Tags == Other.Tags &&
			bHasEditorData == Other.bHasEditorData &&
			bRequiresCookedData == Other.bRequiresCookedData;
}

bool FConcertClientInfo::operator!=(const FConcertClientInfo& Other) const
{
	return !operator==(Other);
}

FText FConcertSessionClientInfo::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLine(ClientInfo.ToDisplayString());
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionClientInfo", "ClientEndpointId", "Client Endpoint ID: {0}"), FText::FromString(ClientEndpointId.ToString()));
	return TextBuilder.ToText();
}

FText FConcertSessionInfo::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "SessionId", "Session ID: {0}"), FText::FromString(SessionId.ToString()));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "SessionName", "Session Name: {0}"), FText::FromString(SessionName));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "OwnerUserName", "Session Owner: {0}"), FText::FromString(OwnerUserName));
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "ProjectName", "Session Project: {0}"), FText::FromString(Settings.ProjectName));
	//TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "BaseRevision", "Session Base Revision: {0}"), FText::AsNumber(Settings.BaseRevision, &FNumberFormattingOptions::DefaultNoGrouping()));
	if (VersionInfos.Num() > 0)
	{
		const FConcertSessionVersionInfo& VersionInfo = VersionInfos.Last();
		TextBuilder.AppendLineFormat(
			NSLOCTEXT("ConcertSessionInfo", "EngineVersion", "Session Engine Version: {0}.{1}.{2}-{3}"), 
			FText::AsNumber(VersionInfo.EngineVersion.Major, &FNumberFormattingOptions::DefaultNoGrouping()),
			FText::AsNumber(VersionInfo.EngineVersion.Minor, &FNumberFormattingOptions::DefaultNoGrouping()),
			FText::AsNumber(VersionInfo.EngineVersion.Patch, &FNumberFormattingOptions::DefaultNoGrouping()),
			FText::AsNumber(VersionInfo.EngineVersion.Changelist, &FNumberFormattingOptions::DefaultNoGrouping())
			);
	}
	TextBuilder.AppendLineFormat(NSLOCTEXT("ConcertSessionInfo", "ServerEndpointId", "Server Endpoint ID: {0}"), FText::FromString(ServerEndpointId.ToString()));
	return TextBuilder.ToText();
}

bool FConcertSessionFilter::ActivityIdPassesFilter(const int64 InActivityId) const
{
	if (ActivityIdsToInclude.Contains(InActivityId))
	{
		return true;
	}

	if (ActivityIdsToExclude.Contains(InActivityId))
	{
		return false;
	}

	return ActivityIdLowerBound <= InActivityId
		&& ActivityIdUpperBound >= InActivityId;
}

namespace UE::Concert::Compression
{

static TAutoConsoleVariable<int32> CVarCompressionType(
	TEXT("Concert.SetCompressionType"), 1,
	TEXT("Specify the type of compression to use when serializing data. A value of 0 means compression is off. A value of 1 = Oodle. All other values = Zlib."));

static TAutoConsoleVariable<int32> CVarCompressionSizeLimit(
	TEXT("Concert.SetCompressionSizeLimit"), 32,
	TEXT("The compressor incurs a performance cost and will only be used if the package size is less than the amount specified (default 32 MB). A value of 0 or less will always compress."));

static TAutoConsoleVariable<int32> CVarCompressionFlags(
	TEXT("Concert.SetCompressionFlags"), 0,
	TEXT("Specify the flags to use when compression is enabled. A value of 0 means no flags. A value of 1 favors smaller sizes. Any other value favors faster encoding."));

int32 GetConsoleVariableCompressionType()
{
	return CVarCompressionType.GetValueOnAnyThread();
}

int32 GetConsoleVariableCompressionFlags()
{
	return CVarCompressionFlags.GetValueOnAnyThread();
}

int32 GetConsoleVariableCompressionSizeLimit()
{
	return CVarCompressionSizeLimit.GetValueOnAnyThread();
}

}

namespace PayloadDetail
{

bool ShouldCompress(const FConcertSessionSerializedPayload& InPayload, EConcertPayloadCompressionType CompressionType)
{
	if (CompressionType == EConcertPayloadCompressionType::None)
	{
		return false;
	}
	if (CompressionType == EConcertPayloadCompressionType::Heuristic)
	{
		return UE::Concert::Compression::ShouldCompress(InPayload.PayloadSize);
	}
	check(CompressionType == EConcertPayloadCompressionType::Always);
	// Otherwise we are always compressing
	return InPayload.PayloadSize > 0;
}

bool TryCompressImpl(const UScriptStruct* InEventType, const void* InEventData, FConcertSessionSerializedPayload& InOutPayload, EConcertPayloadCompressionType CompressionType)
{
	InOutPayload.PayloadSize = InOutPayload.PayloadBytes.Bytes.Num();

	// if we serialized something, compress it
	if (ShouldCompress(InOutPayload, CompressionType))
	{
		TArray<uint8> &InBytes = InOutPayload.PayloadBytes.Bytes;
		TArray<uint8> OutCompressedData;
		// Compress the result to send on the wire
		FName NamedCompressionAlgo = UE::Concert::Compression::GetCompressionAlgorithm();
		ECompressionFlags CompressFlags = UE::Concert::Compression::GetCompressionFlags();
		int32 CompressedSize = FCompression::CompressMemoryBound(NamedCompressionAlgo, InOutPayload.PayloadSize, CompressFlags);
		OutCompressedData.SetNumUninitialized(CompressedSize);
		SCOPED_CONCERT_TRACE(SerializePayload_CompressMemory);
		if (FCompression::CompressMemory(NamedCompressionAlgo, OutCompressedData.GetData(), CompressedSize, InBytes.GetData(), InBytes.Num(), CompressFlags))
		{
			const FNumericUnit<uint64> InBytesDisplay = FUnitConversion::QuantizeUnitsToBestFit(InBytes.Num(), EUnit::Bytes);
			const FNumericUnit<uint64> OutBytesDisplay = FUnitConversion::QuantizeUnitsToBestFit(CompressedSize, EUnit::Bytes);
			UE_LOGF(LogConcert, VeryVerbose, "Compressed payload for %ls from to %lld %ls to %lld %ls. Compression Type: %ls", *InEventType->GetName(), InBytesDisplay.Value,
				   FUnitConversion::GetUnitDisplayString(InBytesDisplay.Units), OutBytesDisplay.Value, FUnitConversion::GetUnitDisplayString(OutBytesDisplay.Units),
				   *NamedCompressionAlgo.ToString());   
			
			OutCompressedData.SetNum(CompressedSize, EAllowShrinking::No);
			InOutPayload.PayloadBytes.Bytes = MoveTemp(OutCompressedData);
			InOutPayload.PayloadCompressionMethod = UE::Concert::Compression::GetCompressionFromNamedType(NamedCompressionAlgo);
			InOutPayload.PayloadCompressionBias = UE::Concert::Compression::GetConcertCompressionBias(CompressFlags);
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "Unable to compress data for %ls!", *InEventType->GetName());
			InOutPayload.PayloadCompressionMethod = EConcertCompressionMethod::Uncompressed;
			InOutPayload.PayloadCompressionBias = EConcertCompressionBias::None;
		}
	}
	else
	{
		const FNumericUnit<uint64> InBytesDisplay = FUnitConversion::QuantizeUnitsToBestFit(InOutPayload.PayloadBytes.Bytes.Num(), EUnit::Bytes);
		UE_LOGF(LogConcert, VeryVerbose, "Not compressing payload for %ls. Size is %lld %ls",
			   *InEventType->GetName(), InBytesDisplay.Value, FUnitConversion::GetUnitDisplayString(InBytesDisplay.Units));

		InOutPayload.PayloadCompressionMethod = EConcertCompressionMethod::Uncompressed;
		InOutPayload.PayloadCompressionBias = EConcertCompressionBias::None;
	}
			
	// Since we can support uncompressed or compressed data this is always successful.
	return true;
}

using OptionalDecompressBytes = TOptional<TArray<uint8>>;

OptionalDecompressBytes DecompressImpl(const FConcertSessionSerializedPayload& InPayload)
{
	EConcertCompressionMethod CompressAlgo = InPayload.PayloadCompressionMethod;
	if (UE::Concert::Compression::DataIsCompressed(CompressAlgo))
	{
		const TArray<uint8> &InBytes = InPayload.PayloadBytes.Bytes;
		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(InPayload.PayloadSize);
		SCOPED_CONCERT_TRACE(DeserializePayload_UncompressMemory);
		ECompressionFlags CompressFlags = UE::Concert::Compression::GetCoreCompressionFlags(InPayload.PayloadCompressionBias);
		FName CompressType = UE::Concert::Compression::GetCompressionAlgorithm(CompressAlgo);
		if (FCompression::UncompressMemory(CompressType, UncompressedData.GetData(), UncompressedData.Num(), InBytes.GetData(), InBytes.Num(), CompressFlags))
		{
			return OptionalDecompressBytes(MoveTemp(UncompressedData));
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "Unable to uncompress data for %ls!", *InPayload.PayloadTypeName.ToString());
		}
	}

	return OptionalDecompressBytes{};
}

bool SerializeImpl(const UScriptStruct* InSourceEventType, const void* InSourceEventData, FConcertSessionSerializedPayload& OutSerializedData)
{
	if (OutSerializedData.SerializationMethod == EConcertPayloadSerializationMethod::Cbor)
	{
		SCOPED_CONCERT_TRACE(ConcertPayload_SerializeBinaryCbor);
		FMemoryWriter Writer(OutSerializedData.PayloadBytes.Bytes);
		FCborStructSerializerBackend Serializer(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(InSourceEventData, *const_cast<UScriptStruct*>(InSourceEventType), Serializer);
		return !Writer.GetError();
	}

	SCOPED_CONCERT_TRACE(ConcertPayload_SerializeBinary);
	FConcertIdentifierWriter Archive(nullptr, OutSerializedData.PayloadBytes.Bytes);
	Archive.SetWantBinaryPropertySerialization(true);
	const_cast<UScriptStruct*>(InSourceEventType)->SerializeItem(Archive, (uint8*)InSourceEventData, nullptr);
	return !Archive.GetError();
}

bool DeserializeImpl(const UScriptStruct* InTargetEventType, void* InOutTargetEventData, EConcertPayloadSerializationMethod SerializeMethod, const TArray<uint8>& InBytes)
{
	if (SerializeMethod == EConcertPayloadSerializationMethod::Cbor)
	{
		SCOPED_CONCERT_TRACE(ConcertPayload_DeserializeBinaryCbor);
		FMemoryReader Reader(InBytes);
		FCborStructDeserializerBackend Deserializer(Reader);
		return FStructDeserializer::Deserialize(InOutTargetEventData, *const_cast<UScriptStruct*>(InTargetEventType), Deserializer) && !Reader.GetError();
	}

	SCOPED_CONCERT_TRACE(ConcertPayload_DeserializeBinary);
	FConcertIdentifierReader Archive(nullptr, InBytes);
	Archive.SetWantBinaryPropertySerialization(true);
	const_cast<UScriptStruct*>(InTargetEventType)->SerializeItem(Archive, (uint8*)InOutTargetEventData, nullptr);
	return !Archive.GetError();
}

bool DeserializeAndDecompress(const UScriptStruct* InTargetEventType, void* InOutTargetEventData, const FConcertSessionSerializedPayload& InPayload)
{
	OptionalDecompressBytes DecompressedBytes = DecompressImpl(InPayload);
	EConcertCompressionMethod CompressAlgo = InPayload.PayloadCompressionMethod;

	if ( UE::Concert::Compression::DataIsCompressed(CompressAlgo) && !DecompressedBytes.IsSet() )
	{
		return false;
	}
	const TArray<uint8>& ByteStream = DecompressedBytes.IsSet() ? DecompressedBytes.GetValue() : InPayload.PayloadBytes.Bytes;
	return DeserializeImpl(InTargetEventType, InOutTargetEventData, InPayload.SerializationMethod, ByteStream);
}

bool SerializePreChecks(const UScriptStruct* InSourceEventType, const void* InSourceEventData, FConcertSessionSerializedPayload& OutSerializedData)
{
	OutSerializedData.PayloadSize = 0;
	OutSerializedData.PayloadBytes.Bytes.Reset();

	return InSourceEventType && InSourceEventData;
}

bool DeserializePreChecks(const UScriptStruct* InEventType, void* InOutEventData,  const FConcertSessionSerializedPayload& Payload)
{
	return InEventType && InOutEventData;
}

} // namespace PayloadDetail

bool FConcertSessionSerializedPayload::SetPayload(const FStructOnScope& InPayload, EConcertPayloadCompressionType CompressionType)
{
	const UStruct* PayloadStruct = InPayload.GetStruct();
	check(PayloadStruct->IsA<UScriptStruct>());
	return SetPayload((UScriptStruct*)PayloadStruct, InPayload.GetStructMemory(), CompressionType);
}

bool FConcertSessionSerializedPayload::SetPayload(const UScriptStruct* InPayloadType, const void* InPayloadData, EConcertPayloadCompressionType CompressionType)
{
	check(InPayloadType && InPayloadData);
	PayloadTypeName = *InPayloadType->GetPathName();
	return PayloadDetail::SerializePreChecks(InPayloadType, InPayloadData, *this)
		&& PayloadDetail::SerializeImpl(InPayloadType, InPayloadData, *this)
		&& PayloadDetail::TryCompressImpl(InPayloadType, InPayloadData, *this, CompressionType);
}

bool FConcertSessionSerializedPayload::GetPayload(FStructOnScope& OutPayload) const
{
	const UStruct* PayloadType = FindObject<UStruct>(nullptr, *PayloadTypeName.ToString());
	if (PayloadType)
	{
		OutPayload.Initialize(PayloadType);
		const UStruct* PayloadStruct = OutPayload.GetStruct();
		check(PayloadStruct->IsA<UScriptStruct>());
		return PayloadDetail::DeserializePreChecks((UScriptStruct*)PayloadStruct, OutPayload.GetStructMemory(), *this)
			&& PayloadDetail::DeserializeAndDecompress((UScriptStruct*)PayloadStruct, OutPayload.GetStructMemory(), *this);
	}
	return false;
}

bool FConcertSessionSerializedPayload::GetPayload(const UScriptStruct* InPayloadType, void* InOutPayloadData) const
{
	check(InPayloadType && InOutPayloadData);
	return IsTypeChildOf(InPayloadType)
		&& PayloadDetail::DeserializePreChecks((UScriptStruct*)InPayloadType, InOutPayloadData, *this)
		&& PayloadDetail::DeserializeAndDecompress((UScriptStruct*)InPayloadType, InOutPayloadData, *this);
}

bool FConcertSessionSerializedPayload::IsTypeChildOf(const UScriptStruct* InPayloadType) const
{
	const UStruct* PayloadType = FindObject<UStruct>(nullptr, *PayloadTypeName.ToString());
	return PayloadType && InPayloadType->IsChildOf(PayloadType);
}

