// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerEvents.h"

ConcertServerEvents::FOnLiveSessionCreated& ConcertServerEvents::OnLiveSessionCreated()
{
	static FOnLiveSessionCreated Event;
	return Event;
}

ConcertServerEvents::FOnArchivedSessionCreated& ConcertServerEvents::OnArchivedSessionCreated()
{
	static FOnArchivedSessionCreated Event;
	return Event;
}

ConcertServerEvents::FOnLiveSessionDestroyed& ConcertServerEvents::OnLiveSessionDestroyed()
{
	static FOnLiveSessionDestroyed Event;
	return Event;
}

ConcertServerEvents::FOnArchivedSessionDestroyed& ConcertServerEvents::OnArchivedSessionDestroyed()
{
	static FOnArchivedSessionDestroyed Event;
	return Event;
}

ConcertServerEvents::FArchiveSession_WithSession& ConcertServerEvents::ArchiveSession_WithSession()
{
	static FArchiveSession_WithSession Event;
	return Event;
}

ConcertServerEvents::FArchiveSession_WithWorkingDir& ConcertServerEvents::ArchiveSession_WithWorkingDir()
{
	static FArchiveSession_WithWorkingDir Event;
	return Event;
}

ConcertServerEvents::FCopySession& ConcertServerEvents::CopySession()
{
	static FCopySession Event;
	return Event;
}

ConcertServerEvents::FExportSession& ConcertServerEvents::ExportSession()
{
	static FExportSession Event;
	return Event;
}

ConcertServerEvents::FRestoreSession& ConcertServerEvents::RestoreSession()
{
	static FRestoreSession Event;
	return Event;
}

ConcertServerEvents::FOnLiveSessionRenamed& ConcertServerEvents::OnLiveSessionRenamed()
{
	static FOnLiveSessionRenamed Event;
	return Event;
}

ConcertServerEvents::FOnArchivedSessionRenamed& ConcertServerEvents::OnArchivedSessionRenamed()
{
	static FOnArchivedSessionRenamed Event;
	return Event;
}