// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSession.h"
#include "ConcertLogGlobal.h"
#include "UObject/StructOnScope.h"
#include "Scratchpad/ConcertScratchpad.h"

FConcertSessionCommonImpl::FConcertSessionCommonImpl(const FConcertSessionInfo& InSessionInfo)
	: SessionInfo(InSessionInfo)
{
}

void FConcertSessionCommonImpl::CommonStartup()
{
	// Create your local scratchpad
	Scratchpad = MakeShared<FConcertScratchpad, ESPMode::ThreadSafe>();
}

void FConcertSessionCommonImpl::CommonShutdown()
{
	// Reset your scratchpad
	Scratchpad.Reset();
}

TArray<FGuid> FConcertSessionCommonImpl::CommonGetSessionClientEndpointIds() const
{
	TArray<FGuid> EndpointIds;
	SessionClients.GenerateKeyArray(EndpointIds);
	return EndpointIds;
}

TArray<FConcertSessionClientInfo> FConcertSessionCommonImpl::CommonGetSessionClients() const
{
	TArray<FConcertSessionClientInfo> ClientInfos;
	ClientInfos.Reserve(SessionClients.Num());
	for (const auto& SessionClientPair : SessionClients)
	{
		ClientInfos.Add(SessionClientPair.Value.ClientInfo);
	}
	return ClientInfos;
}

bool FConcertSessionCommonImpl::CommonFindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const
{
	if (const FSessionClient* FoundSessionClient = SessionClients.Find(EndpointId))
	{
		OutSessionClientInfo = FoundSessionClient->ClientInfo;
		return true;
	}
	return false;
}

FConcertScratchpadRef FConcertSessionCommonImpl::CommonGetScratchpad() const
{
	return Scratchpad.ToSharedRef();
}

FConcertScratchpadPtr FConcertSessionCommonImpl::CommonGetClientScratchpad(const FGuid& ClientEndpointId) const
{
	if (const FSessionClient* FoundSessionClient = SessionClients.Find(ClientEndpointId))
	{
		return FoundSessionClient->Scratchpad;
	}
	return nullptr;
}

FDelegateHandle FConcertSessionCommonImpl::CommonRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler)
{
	CustomEventHandlers.FindOrAdd(EventMessageType).Add(Handler);
	return Handler->GetHandle();
}

void FConcertSessionCommonImpl::CommonUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle)
{
	TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
	if (HandlerArrayPtr)
	{
		for (auto It = HandlerArrayPtr->CreateIterator(); It; ++It)
		{
			if ((*It)->GetHandle() == EventHandle)
			{
				It.RemoveCurrent();
				break;
			}
		}
	}
}

void FConcertSessionCommonImpl::CommonUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler)
{
	TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
	if (HandlerArrayPtr)
	{
		HandlerArrayPtr->RemoveAllSwap([EventHandler](const TSharedPtr<IConcertSessionCustomEventHandler>& InHandler) -> bool
		{
			return InHandler->HasSameObject(EventHandler);
		});
	}
}

void FConcertSessionCommonImpl::CommonClearCustomEventHandler(const FName& EventMessageType)
{
	CustomEventHandlers.Remove(EventMessageType);
}

void FConcertSessionCommonImpl::CommonHandleCustomEvent(const FConcertMessageContext& Context)
{
	SCOPED_CONCERT_TRACE(ConcertSession_HandleCustomEvent);

	const FConcertSession_CustomEvent* Message = Context.GetMessage<FConcertSession_CustomEvent>();
	FConcertScratchpadPtr SenderScratchpad = CommonGetClientScratchpad(Message->SourceEndpointId);

	// Attempt to deserialize the payload
	FStructOnScope RawPayload;
	if (Message->SerializedPayload.GetPayload(RawPayload))
	{
		// Dispatch to external handlers
		TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(RawPayload.GetStruct()->GetFName());
		if (HandlerArrayPtr)
		{
			FConcertSessionContext SessionContext{ Message->SourceEndpointId, Message->MessageId, Message->GetMessageFlags(), SenderScratchpad };
			for (const TSharedPtr<IConcertSessionCustomEventHandler>& Handler : *HandlerArrayPtr)
			{
				Handler->HandleEvent(SessionContext, RawPayload.GetStructMemory());
			}
		}
		else
		{
			UE_LOGF(LogConcert, Verbose, "No event handler for '%ls' has been registered for session '%ls'. This event will be ignored!", *RawPayload.GetStruct()->GetName(), *SessionInfo.SessionName);
		}
	}
}

void FConcertSessionCommonImpl::CommonRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler)
{
	UE_CLOGF(CustomRequestHandlers.Contains(RequestMessageType), LogConcert, Warning, "A request handler for '%ls' had already been registered for session '%ls'. The new handler will replace the existing one!", *RequestMessageType.ToString(), *SessionInfo.SessionName);
	CustomRequestHandlers.Add(RequestMessageType, Handler);
}

void FConcertSessionCommonImpl::CommonUnregisterCustomRequestHandler(const FName& RequestMessageType)
{
	CustomRequestHandlers.Remove(RequestMessageType);
}

TFuture<FConcertSession_CustomResponse> FConcertSessionCommonImpl::CommonHandleCustomRequest(const FConcertMessageContext& Context)
{
	const FConcertSession_CustomRequest* Message = Context.GetMessage<FConcertSession_CustomRequest>();

	FConcertScratchpadPtr SenderScratchpad = CommonGetClientScratchpad(Message->SourceEndpointId);

	// Default response
	FConcertSession_CustomResponse ResponseData;
	ResponseData.ResponseCode = EConcertResponseCode::UnknownRequest;

	// Attempt to deserialize the payload
	FStructOnScope RawPayload;
	if (Message->SerializedPayload.GetPayload(RawPayload))
	{
		// Dispatch to external handler
		TSharedPtr<IConcertSessionCustomRequestHandler> Handler = CustomRequestHandlers.FindRef(RawPayload.GetStruct()->GetFName());
		if (Handler.IsValid())
		{
			FStructOnScope ResponsePayload(Handler->GetResponseType());
			FConcertSessionContext SessionContext{ Message->SourceEndpointId, Message->MessageId, Message->GetMessageFlags(), SenderScratchpad };
			ResponseData.SetResponseCode(Handler->HandleRequest(SessionContext, RawPayload.GetStructMemory(), ResponsePayload.GetStructMemory()));
			if (ResponseData.ResponseCode == EConcertResponseCode::Success || ResponseData.ResponseCode == EConcertResponseCode::Failed)
			{
				ResponseData.SerializedPayload.SetPayload(ResponsePayload);
			}
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "No request handler for '%ls' has been registered for session '%ls'. This request will be ignored!", *RawPayload.GetStruct()->GetName(), *SessionInfo.SessionName);
		}
	}

	return FConcertSession_CustomResponse::AsFuture(MoveTemp(ResponseData));
}

bool FConcertSessionCommonImpl::CommonBuildCustomEvent(const UScriptStruct* EventType, const void* EventData, const FGuid& SourceEndpointId, const TArray<FGuid>& DestinationEndpointIds, FConcertSession_CustomEvent& OutCustomEvent)
{
	bool bSuccess = true;

	// Serialize the event
	bSuccess &= OutCustomEvent.SerializedPayload.SetPayload(EventType, EventData);

	// Set the source endpoint
	OutCustomEvent.SourceEndpointId = SourceEndpointId;

	// Set the destination endpoints
	OutCustomEvent.DestinationEndpointIds = DestinationEndpointIds;

	return bSuccess;
}

bool FConcertSessionCommonImpl::CommonBuildCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& SourceEndpointId, const FGuid& DestinationEndpointId, FConcertSession_CustomRequest& OutCustomRequest)
{
	bool bSuccess = true;

	// Serialize the request
	bSuccess &= OutCustomRequest.SerializedPayload.SetPayload(RequestType, RequestData);

	// Set the source endpoint
	OutCustomRequest.SourceEndpointId = SourceEndpointId;

	// Set the destination endpoint
	OutCustomRequest.DestinationEndpointId = DestinationEndpointId;

	return bSuccess;
}

void FConcertSessionCommonImpl::CommonHandleCustomResponse(const FConcertSession_CustomResponse& Response, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler)
{
	// Successful response?
	if (Response.ResponseCode != EConcertResponseCode::Success)
	{
		Handler->HandleResponse(nullptr);
		return;
	}

	// Attempt to deserialize the payload
	FStructOnScope ResponseRawPayload;
	if (!Response.SerializedPayload.GetPayload(ResponseRawPayload))
	{
		Handler->HandleResponse(nullptr);
		return;
	}

	// Dispatch to external handler
	Handler->HandleResponse(ResponseRawPayload.GetStructMemory());
}
