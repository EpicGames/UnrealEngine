// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSettings.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertSettings)

#define LOCTEXT_NAMESPACE "ConcertSettingsUtils"

namespace ConcertSettingsUtils
{

FText ValidateFileSharingPath(const FString& Path)
{
	if (Path.IsEmpty())
	{
		return LOCTEXT("PathName_Empty", "Path name is empty.");
	}

	FText Reason;
	if (!FPaths::ValidatePath(Path, &Reason))
	{
		return FText::Format(LOCTEXT("IllegalPathName", "{0}"), Reason);
	}

	if (!FPaths::DirectoryExists(Path))
	{
		return FText::Format(LOCTEXT("DirectoryDoesNotExist", "Path {0} does not exist."), FText::FromString(Path));
	}

	return FText::GetEmpty();
}
	
	
/** Returns an error messages if the user display name is invalid, otherwise, returns an empty text. */
FText ValidateDisplayName(const FString& Name)
{
	constexpr int32 MaxDisplayNameLen = 32;

	if (Name.Len() > MaxDisplayNameLen)
	{
		return FText::Format(LOCTEXT("DisplayName_TooLong", "Too long. Limited to {0} characters!"), MaxDisplayNameLen);
	}

	return FText::GetEmpty();
}

/** Returns an error messages if the specified session name is invalid, otherwise, returns an empty text. */
FText ValidateSessionName(const FString& Name)
{
	constexpr int32 MaxSessionNameLen = 128;

	if (Name.IsEmpty())
	{
		return LOCTEXT("SessionName_EmptyNameError", "Name cannot be left blank!");
	}
	else if (FChar::IsWhitespace(Name[0]))
	{
		return LOCTEXT("SessionName_IllegalBlankLeading", "Illegal leading white space!");
	}
	else if (FChar::IsWhitespace(Name[Name.Len() - 1]))
	{
		return LOCTEXT("SessionName_IllegalBlankTrailing", "Illegal trailing white space!");
	}
	else if (Name.Len() > MaxSessionNameLen)
	{
		return FText::Format(LOCTEXT("SessionName_TooLong", "Too long. Limited to {0} characters!"), MaxSessionNameLen);
	}

	FText Reason;
	if (!FPaths::ValidatePath(Name, &Reason))
	{
		return FText::Format(LOCTEXT("IllegalNameError", "{0}"), Reason);
	}

	return FText::GetEmpty(); // No Error.
}

} // ConcertSettingsUtils

#undef LOCTEXT_NAMESPACE

