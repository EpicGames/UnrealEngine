// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTransportEvents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertTransportEvents)

namespace ConcertTransportEvents
{
	FConcertTransportLoggingEnabledChanged& OnConcertTransportLoggingEnabledChangedEvent()
	{
		static FConcertTransportLoggingEnabledChanged Instance;
		return Instance;
	}
	
	FConcertClientLogEvent& OnConcertClientLogEvent()
	{
		static FConcertClientLogEvent Instance;
		return Instance;
	}

	FConcertServerLogEvent& OnConcertServerLogEvent()
	{
		static FConcertServerLogEvent Instance;
		return Instance;
	}
}

