// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertUtil.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "ConcertLogGlobal.h"
#include "ConcertLogger.h"
#include "IConcertSession.h"

bool ConcertUtil::DeleteDirectoryTree(const TCHAR* InDirectoryToDelete, const TCHAR* InMoveToDirBeforeDelete)
{
	SCOPED_CONCERT_TRACE(ConcertUtil_DeleteDirectoryTree);
	IFileManager& FileManager = IFileManager::Get();

	if (FileManager.DirectoryExists(InDirectoryToDelete))
	{
		// HACK: Move/rename the directory first (very fast if on the same file system) to prevent other threads/processes to scan/access it while the system is taking long time to delete it.
		const FString MoveDir = InMoveToDirBeforeDelete && FileManager.DirectoryExists(InMoveToDirBeforeDelete) ? FString(InMoveToDirBeforeDelete) : FPaths::ProjectIntermediateDir();
		const FString TempDirToDelete = MoveDir / FString::Printf(TEXT("__Concert_%s"), *FGuid::NewGuid().ToString());

		// Try to Move/rename first. (This may fail if 'TempDirToDelete' path was too long for example)
		FString DirToDelete = FileManager.Move(*TempDirToDelete, InDirectoryToDelete, true, true, true, false) ? TempDirToDelete : InDirectoryToDelete;

		// Try deleting the directory.
		return FileManager.DeleteDirectory(*DirToDelete, false, true);
	}

	return true;
}

bool ConcertUtil::Copy(FArchive& DstAr, FArchive& SrcAr, int64 Size)
{
	SCOPED_CONCERT_TRACE(ConcertUtil_Copy);
	check(DstAr.IsSaving() && SrcAr.IsLoading());

	constexpr int64 DataChunkSize = 4 * 1024; // 4K is a typical low level block size.
	TArray<uint8> DataChunk;
	DataChunk.AddUninitialized(static_cast<int32>(DataChunkSize));

	// Copy full 4K block from the source to dest.
	int64 ChunkCount = Size / DataChunkSize;
	while (ChunkCount > 0)
	{
		SrcAr.Serialize(DataChunk.GetData(), DataChunkSize); // Read.
		DstAr.Serialize(DataChunk.GetData(), DataChunkSize); // Write.
		--ChunkCount;
	}

	// Copy the last few bytes remaining.
	int64 Remaining = Size % DataChunkSize;
	SrcAr.Serialize(DataChunk.GetData(), Remaining); // Read.
	DstAr.Serialize(DataChunk.GetData(), Remaining); // Write.

	return !SrcAr.IsError() && !DstAr.IsError();
}

void ConcertUtil::SetVerboseLogging(bool bInState)
{
	FConcertLogger::SetVerboseLogging(bInState);
}

IConcertTransportLoggerRef ConcertUtil::CreateLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc)
{
	return FConcertLogger::CreateLogger(InOwnerContext, MoveTemp(LogListenerFunc));
}
