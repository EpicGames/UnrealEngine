// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertVersion.h"
#include "ConcertSettings.h"

#include "Misc/EngineVersion.h"
#include "Modules/BuildVersion.h"
#include "Serialization/CustomVersion.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertVersion)

LLM_DEFINE_TAG(Concert_ConcertVersion);
#define LOCTEXT_NAMESPACE "ConcertVersion"

namespace ConcertVersionUtil
{
static TAutoConsoleVariable<int32> CVarVersionValidation(
	TEXT("Concert.VersionValidationMode"), 0,
	TEXT("Specifies if we are using default or legacy based version validation. In default mode (0), clients will be allowed to join sessions if patch versions are compatible. For example, 5.4.4 is version compatible with 5.4.3. In legacy mode (1), versions have to match exactly when joining a session."));

bool ValidateVersion(const int32 InCurrent, const int32 InOther, const FText InVersionDisplayName, const EConcertVersionValidationMode InValidationMode, bool bIsVersionPatchValue, FText* OutFailureReason)
{
	// Convert the InValidationMode into something we can handle.  If we are in default version mode then we need to convert patch compatible version to ::Identical or ::Compatible.
	// 
	const EConcertVersionValidationMode Mode = [bIsVersionPatchValue, InValidationMode]()
	{
		const bool bUseDefaultVersionValidation = CVarVersionValidation.GetValueOnAnyThread() == 0;
		if (bUseDefaultVersionValidation && bIsVersionPatchValue && InValidationMode == EConcertVersionValidationMode::PatchCompatible)
		{
			return EConcertVersionValidationMode::Compatible;
		}
		else if (InValidationMode == EConcertVersionValidationMode::PatchCompatible)
		{
			return EConcertVersionValidationMode::Identical;
		}
		return InValidationMode;
	}();
	
	switch (Mode)
	{
	case EConcertVersionValidationMode::Identical:
		if (InOther != InCurrent)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(LOCTEXT("Error_InvalidIdenticalVersionFmt", "Invalid version for '{0}' (expected '{1}', got '{2}')"), InVersionDisplayName, FText::AsNumber(InCurrent, &FNumberFormattingOptions::DefaultNoGrouping()), FText::AsNumber(InOther, &FNumberFormattingOptions::DefaultNoGrouping()));
			}
			return false;
		}
		break;

	case EConcertVersionValidationMode::Compatible:
		if (InOther < InCurrent)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(LOCTEXT("Error_InvalidCompatibleVersionFmt", "Invalid version for '{0}' (expected '{1}' or greater, got '{2}')"), InVersionDisplayName, FText::AsNumber(InCurrent, &FNumberFormattingOptions::DefaultNoGrouping()), FText::AsNumber(InOther, &FNumberFormattingOptions::DefaultNoGrouping()));
			}
			return false;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EConcertVersionValidationMode!"));
		break;
	}
	
	return true;
}

} // namespace ConcertVersionUtil


void FConcertFileVersionInfo::Initialize()
{
	FileVersion = GPackageFileUEVersion.FileVersionUE4;
	FileVersionUE5 = GPackageFileUEVersion.FileVersionUE5;

	FileVersionLicensee = GPackageFileLicenseeUEVersion;
}

bool FConcertFileVersionInfo::Validate(const FConcertFileVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason) const
{
	const bool bIsVersionPatchValue = false;
	
	return ConcertVersionUtil::ValidateVersion(FileVersion, InOther.FileVersion, LOCTEXT("UE4PackageVersionName", "UE4 Package Version"), InValidationMode, bIsVersionPatchValue, OutFailureReason)
		&& ConcertVersionUtil::ValidateVersion(FileVersionUE5, InOther.FileVersionUE5, LOCTEXT("UE5PackageVersionName", "UE5 Package Version"), InValidationMode, bIsVersionPatchValue, OutFailureReason)
		&& ConcertVersionUtil::ValidateVersion(FileVersionLicensee, InOther.FileVersionLicensee, LOCTEXT("LicenseePackageVersionName", "Licensee Package Version"), InValidationMode, bIsVersionPatchValue, OutFailureReason);
}


void FConcertEngineVersionInfo::Initialize(const FEngineVersion& InVersion)
{
	Major = InVersion.GetMajor();
	Minor = InVersion.GetMinor();
	Patch = InVersion.GetPatch();
	Changelist = InVersion.GetChangelist();
}

bool FConcertEngineVersionInfo::Validate(const FConcertEngineVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason) const
{
	return ConcertVersionUtil::ValidateVersion(Major, InOther.Major, LOCTEXT("MajorEngineVersionName", "Major Engine Version"), InValidationMode, false /*bIsVersionPatchValue*/, OutFailureReason)
		&& ConcertVersionUtil::ValidateVersion(Minor, InOther.Minor, LOCTEXT("MinorEngineVersionName", "Minor Engine Version"), InValidationMode, false /*bIsVersionPatchValue*/, OutFailureReason)
		&& ConcertVersionUtil::ValidateVersion(Patch, InOther.Patch, LOCTEXT("PatchEngineVersionName", "Patch Engine Version"), InValidationMode, true /*bIsVersionPatchValue*/, OutFailureReason)
		&& ConcertVersionUtil::ValidateVersion(Changelist, InOther.Changelist, LOCTEXT("ChangelistEngineVersionName", "Changelist Engine Version"), InValidationMode, true /*bIsVersionPatchValue*/, OutFailureReason);
}


void FConcertCustomVersionInfo::Initialize(const FCustomVersion& InVersion)
{
	FriendlyName = InVersion.GetFriendlyName();
	Key = InVersion.Key;
	Version = InVersion.Version;
}

bool FConcertCustomVersionInfo::Validate(const FConcertCustomVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason) const
{
	check(Key == InOther.Key);
	const bool bIsVersionPatchValue = false;
	
	return ConcertVersionUtil::ValidateVersion(Version, InOther.Version, FText::AsCultureInvariant(FriendlyName.IsNone() ? Key.ToString() : FriendlyName.ToString()), InValidationMode, bIsVersionPatchValue, OutFailureReason);
}

void FConcertSessionVersionInfo::Initialize(bool bSupportMixedBuildTypes)
{
	LLM_SCOPE_BYTAG(Concert_ConcertVersion);

	FileVersion.Initialize();
	EngineVersion.Initialize(FEngineVersion::Current());

	if (bSupportMixedBuildTypes)
	{
		// For builds synced via UGS, we override the changelist of the engine version with the current build version changelist
		// as this helps to keep the changelists of programmers and artists/designers in-sync when creating and joining sessions
		//	eg) CL# 1 is a code change, and CL# 2 is a content change:
		//	 - A programmer syncing CL# 2 would have an engine version with a CL# of 2 (from building their own editor), and a build version CL# of 2 (from UGS).
		//	 - An artist/designer syncing CL# 2 would have an engine version with a CL# of 1 (from the pre-built editor), but a build version CL# of 2 (from UGS).
		{
			// Read the default data (rather than the executable specific data), as the default data 
			// is updated when syncing, but the executable data is only updated when compiling
			FBuildVersion BuildVersion;
			if (FBuildVersion::TryRead(FBuildVersion::GetDefaultFileName(), BuildVersion))
			{
				// Only apply the build version if our engine changelist is compatible with the synced build
				// If this check fails then it likely means that a programmer synced (updating the build version) 
				// without also compiling their binaries (to update the engine version)
				if (EngineVersion.Changelist >= (uint32)BuildVersion.CompatibleChangelist)
				{
					EngineVersion.Changelist = BuildVersion.Changelist;
				}
			}
		}
	}

	FCustomVersionContainer AllCurrentVersions = FCurrentCustomVersions::GetAll();
	for (const FCustomVersion& EngineCustomVersion : AllCurrentVersions.GetAllVersions())
	{
		FConcertCustomVersionInfo& CustomVersion = CustomVersions.AddDefaulted_GetRef();
		CustomVersion.Initialize(EngineCustomVersion);
	}
}

bool FConcertSessionVersionInfo::Validate(const FConcertSessionVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason) const
{
	if (!FileVersion.Validate(InOther.FileVersion, InValidationMode, OutFailureReason))
	{
		return false;
	}
	
	if (!EngineVersion.Validate(InOther.EngineVersion, InValidationMode, OutFailureReason))
	{
		return false;
	}

	for (const FConcertCustomVersionInfo& CustomVersion : CustomVersions)
	{
		const FConcertCustomVersionInfo* OtherCustomVersion = InOther.CustomVersions.FindByPredicate([&CustomVersion](const FConcertCustomVersionInfo& PotentialCustomVersion)
		{
			return PotentialCustomVersion.Key == CustomVersion.Key;
		});

		if (!OtherCustomVersion)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(LOCTEXT("Error_MissingVersionFmt", "Invalid version for '{0}' (expected '{1}', got '<none>'). Do you have a required plugin disabled?"), FText::AsCultureInvariant(CustomVersion.FriendlyName.IsNone() ? CustomVersion.Key.ToString() : CustomVersion.FriendlyName.ToString()), CustomVersion.Version);
			}
			return false;
		}

		check(OtherCustomVersion && CustomVersion.Key == OtherCustomVersion->Key);
		if (!CustomVersion.Validate(*OtherCustomVersion, InValidationMode, OutFailureReason))
		{
			return false;
		}
	}

	if (InValidationMode == EConcertVersionValidationMode::Identical && InOther.CustomVersions.Num() > CustomVersions.Num())
	{
		// The identical check also requires that there are no extra versions (missing versions would have been caught by the loop above)
		// We only need to bother figuring out which version is extra if we're reporting back error information, as this is already an error condition
		if (OutFailureReason)
		{
			for (const FConcertCustomVersionInfo& OtherCustomVersion : InOther.CustomVersions)
			{
				const FConcertCustomVersionInfo* CustomVersion = CustomVersions.FindByPredicate([&OtherCustomVersion](const FConcertCustomVersionInfo& PotentialCustomVersion)
				{
					return PotentialCustomVersion.Key == OtherCustomVersion.Key;
				});

				if (!CustomVersion)
				{
					*OutFailureReason = FText::Format(LOCTEXT("Error_ExtraCustomVersionFmt", "Invalid version for '{0}' (expected '<none>', got '{1}'). Do you have an extra plugin enabled?"), FText::AsCultureInvariant(OtherCustomVersion.FriendlyName.IsNone() ? OtherCustomVersion.Key.ToString() : OtherCustomVersion.FriendlyName.ToString()), OtherCustomVersion.Version);
					break;
				}
			}
		}
		return false;
	}

	return true;
}

FText FConcertSessionVersionInfo::AsText() const
{
	return FText::Format(
		LOCTEXT("EngineVersionFmt", "{0}.{1}.{2}-{3}"),
		FText::AsNumber(EngineVersion.Major, &FNumberFormattingOptions::DefaultNoGrouping()),
		FText::AsNumber(EngineVersion.Minor, &FNumberFormattingOptions::DefaultNoGrouping()),
		FText::AsNumber(EngineVersion.Patch, &FNumberFormattingOptions::DefaultNoGrouping()),
		FText::AsNumber(EngineVersion.Changelist, &FNumberFormattingOptions::DefaultNoGrouping())
		);
}
#undef LOCTEXT_NAMESPACE

