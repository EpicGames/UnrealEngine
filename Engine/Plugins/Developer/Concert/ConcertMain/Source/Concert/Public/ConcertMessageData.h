// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CompressionFlags.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "ConcertVersion.h"
#include "ConcertSettings.h"
#include "ConcertMessageData.generated.h"

class FStructOnScope;

using FActivityID = int64;

UENUM()
enum class EConcertServerFlags : uint8
{
	None = 0,
	//The server will ignore the session requirement when someone try to join a session
	IgnoreSessionRequirement = 1 << 0,
};
ENUM_CLASS_FLAGS(EConcertServerFlags)


UENUM()
enum class EConcertSessionState : uint8
{
	/** Session is a normal state and can be joined. */
	Normal = 0,

	/** Session is in a transient state and cannot be joined. */
	Transient
};

UENUM()
enum class EConcertPayloadCompressionType : uint8
{
	// The serialized data will not be compressed.
	None = 0,
	// The serialized data will be compressed based on struct size.
	Heuristic,
	// The serialized data will always be compressed.
	Always
};

UENUM()
enum class UE_DEPRECATED(5.8, "Use EConcertCompressionMethod instead.") EConcertCompressionDetails : uint8
{
	/** The package data (the body) is written uncompressed. */
	Uncompressed = 0,

	/** The package data (the body) is written in a single compressed block. The value 1 represents the original format. Still used to store small packages.*/
	Compressed = 1 << 0,

	/** If compression is enabled and this bit is set then oodle compression is used for the package data. */
	CompressWithOodle = 1 << 1,

	/** Compress for speed. */
	CompressForSpeed = 1 << 2,

	/** Compress for smaller sizes. */
	CompressForSize = 1 << 3
};

/** Compression method to use. Note that these are assigned a bit shifted value to match the previous implementation in EConcertCompressionDetails. */
UENUM()
enum class EConcertCompressionMethod : uint8
{
	/** Data is uncompressed. */
	Uncompressed = 0,

	/**  ZLib compression is used for the serialized data. */
	Zlib = 1 << 0,
	
	/** Oodle compression is used for the serialized data. */
	Oodle = 1 << 1
};

/** The bias (if any) to use when compressing the data. Note that these are assigned a bit shifted value to match the previous implementation in EConcertCompressionDetails. */
UENUM()
enum class EConcertCompressionBias : uint8
{
	/** Compress without a bias. */
	None,

	/** Compress with a bias for speed. */
	Speed = 1 << 2,

	/** Compress with a bias for size. */
	Size = 1 << 3
};



namespace UE::Concert::Compression
{

/** Get the compression type from the console variable setting. */
CONCERT_API int32 GetConsoleVariableCompressionType();

/** Function to check the console variable for the compression flags. Speed vs size. */
CONCERT_API int32 GetConsoleVariableCompressionFlags();

/** Function to check the console variable for compression size limit.*/
CONCERT_API int32 GetConsoleVariableCompressionSizeLimit();

/** Based on the size of the data to compress indicate if we should invoke the compressor. */
template <typename SizeType>
inline bool ShouldCompress(SizeType DataSize)
{
	static_assert(std::is_integral<SizeType>::value);
	const int32 CompressionLimit = GetConsoleVariableCompressionSizeLimit();
	const int32 MinLimit = 512;
	if (CompressionLimit <= 0)
	{
		return DataSize > MinLimit;
	}
	const SizeType MaxPackageDataSizeForCompression = static_cast<SizeType>(CompressionLimit) * 1024 * 1024;
	return DataSize > MinLimit && DataSize <= MaxPackageDataSizeForCompression;
}

/** Returns truen if the compression details is set to compressed format.  */
inline bool DataIsCompressed(EConcertCompressionMethod InFormat)
{
	return InFormat == EConcertCompressionMethod::Zlib || InFormat == EConcertCompressionMethod::Oodle;
}

/** Returns true if the compression details is set to uncompressed format. */
inline bool DataIsUncompressed(EConcertCompressionMethod InFormat)
{
	return InFormat == EConcertCompressionMethod::Uncompressed;
}

/** Returns true if we are suppose to use Oodle for compression. */
inline bool DataIsCompressedWithOodle(EConcertCompressionMethod InFormat)
{
	return InFormat == EConcertCompressionMethod::Oodle;
}

/** Get the named compression algorithm to invoke with serializer and memory compressors. */
inline FName GetCompressionAlgorithm()
{
	const int32 CompressionAlgo = GetConsoleVariableCompressionType();
	if (CompressionAlgo == 1)
	{
		return NAME_Oodle;
	}
	return NAME_Zlib;
}

/** Get the default flags to use when invoking the compressor. */
inline ECompressionFlags GetCompressionFlags()
{
	const int32 CompressionFlags = GetConsoleVariableCompressionFlags();
	if (CompressionFlags == 0)
	{
		return COMPRESS_NoFlags;
	}
	if (CompressionFlags == 1)
	{
		return COMPRESS_BiasSize;
	}
	return COMPRESS_BiasSpeed;
}

/** Get the named compression algorithm based on the provided details.  Current this is only Zlib or Oodle. */
inline FName GetCompressionAlgorithm(EConcertCompressionMethod InDetails)
{
	return DataIsCompressedWithOodle(InDetails) ? NAME_Oodle : NAME_Zlib;
}

/** Get the compression enum value given the concert compression settings.  This the value passed into the serializer and memory compressor. */
inline ECompressionFlags GetCoreCompressionFlags(EConcertCompressionBias InBias)
{
	if (InBias == EConcertCompressionBias::Size)
	{
		return COMPRESS_BiasSize;
	}
	if (InBias == EConcertCompressionBias::Speed)
	{
		return COMPRESS_BiasSpeed;
	}
	return COMPRESS_NoFlags;
}

/** Get the concert compression bias value given the compressor compression flags. */
inline EConcertCompressionBias GetConcertCompressionBias(ECompressionFlags InFlags)
{
	if (InFlags == COMPRESS_BiasSize)
	{
		return EConcertCompressionBias::Size;
	}
	
	if (InFlags == COMPRESS_BiasSpeed)
	{
		return EConcertCompressionBias::Speed;
	}
	
	return EConcertCompressionBias::None;
}

/** For a given named compression algorithm and compression flags convert it into a EConcertCompressionMethod */
inline EConcertCompressionMethod GetCompressionFromNamedType(FName NamedMethod)
{
	check(NamedMethod == NAME_Oodle || NamedMethod == NAME_Zlib);

	EConcertCompressionMethod Compressor = EConcertCompressionMethod::Zlib;

	if (NamedMethod == NAME_Oodle)
	{
		Compressor = EConcertCompressionMethod::Oodle;
	}

	return Compressor;
}

/** Based on the data size return the current compression details to store. */
template <typename SizeType>
inline EConcertCompressionMethod GetCompressionMethod(SizeType DataSize)
{
	static_assert(std::is_integral<SizeType>::value);
	const int32 CompressionType = GetConsoleVariableCompressionType();
	if (ShouldCompress(DataSize) && CompressionType != 0)
	{
		if (CompressionType == 1)
		{
			return EConcertCompressionMethod::Oodle;
		}
		return EConcertCompressionMethod::Zlib;
	}
	return EConcertCompressionMethod::Uncompressed;
}

}

UENUM()
enum class EConcertPayloadSerializationMethod : uint8
{
	// The data will be serialized using standard platform method.
	Standard = 0,
	// The data will be serialized using Cbor method.
	Cbor,
};

USTRUCT()
struct FConcertSessionChallengeData 
{
	GENERATED_BODY()

	FConcertSessionChallengeData()
	{
		ChallengeKey = FGuid::NewGuid();
		CreationTimeInTicks = FDateTime::Now().GetTicks();
	}

	bool operator==(const FConcertSessionChallengeData& Other) const = default;
	
	UPROPERTY(Meta = (IgnoreForMemberInitializationTest))
	FGuid ChallengeKey;

	UPROPERTY()
	int64 CreationTimeInTicks;
};

USTRUCT()
struct FConcertSessionFileShareChallenge
{
	GENERATED_BODY()

	UPROPERTY()
	FString Path;

	UPROPERTY()
	FConcertSessionChallengeData Data;
};

USTRUCT()
struct FConcertSessionFileShare
{
	GENERATED_BODY()

	/**
	 * Indicates that file sharing is enabled and available on the server.
	 */
	UPROPERTY()
	bool bFileShareEnabled = false;

	/**
	 * Root path for all file shares for this session.
	 */
	UPROPERTY()
	FString FileShareRoot;

	/**
	 * A challenge id that can be shared with the server / client to confirm that
	 * file sharing can be established between a server and a client.
	 */
	UPROPERTY()
	FConcertSessionFileShareChallenge Challenge;
};

/** Holds info on an instance communicating through concert */
USTRUCT()
struct FConcertInstanceInfo
{
	GENERATED_BODY();

	/** Initialize this instance information based on the current environment */
	CONCERT_API void Initialize();

	/** Create a user friendly display string for using in things such as tooltips. */
	CONCERT_API FText ToDisplayString() const;

	/** Holds the instance identifier. */
	UPROPERTY(VisibleAnywhere, Category="Instance Info")
	FGuid InstanceId;

	/** Holds the instance name. */
	UPROPERTY(VisibleAnywhere, Category="Instance Info")
	FString InstanceName;

	/** Holds the instance type (Editor, Game, Server, etc). */
	UPROPERTY(VisibleAnywhere, Category="Instance Info")
	FString InstanceType; // TODO: enum?

	CONCERT_API bool operator==(const FConcertInstanceInfo& Other) const;
	CONCERT_API bool operator!=(const FConcertInstanceInfo& Other) const;
};

/** Holds info on a Concert server */
USTRUCT()
struct FConcertServerInfo
{
	GENERATED_BODY();

	/** Initialize this server information based on the current environment */
	CONCERT_API void Initialize();

	/** Create a user friendly display string for using in things such as tooltips. */
	CONCERT_API FText ToDisplayString() const;

	/** Server endpoint for performing administration tasks (FConcertAdmin_X messages) */
	UPROPERTY()
	FGuid AdminEndpointId;

	UPROPERTY()
	FString ServerName;

	/** Basic server information */
	UPROPERTY(VisibleAnywhere, Category="Server Info")
	FConcertInstanceInfo InstanceInfo;

	/** Contains information on the server settings */
	UPROPERTY(VisibleAnywhere, Category = "Server Info")
	EConcertServerFlags ServerFlags = EConcertServerFlags::None;
};

/** Holds info on a client connected through concert */
USTRUCT()
struct FConcertClientInfo
{
	GENERATED_BODY()

	/** Initialize this client information based on the current environment */
	CONCERT_API void Initialize();

	/** Create a user friendly display string for using in things such as tooltips. */
	CONCERT_API FText ToDisplayString() const;

	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FConcertInstanceInfo InstanceInfo;

	/** Holds the name of the device that the instance is running on. */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FString DeviceName;

	/** Holds the name of the platform that the instance is running on. */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FString PlatformName;

	/** Holds the name of the user that owns this instance. */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FString UserName;

	/** Holds the display name of the user that owns this instance. */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FString DisplayName;

	/** Holds the color of the user avatar in a session. */
	UPROPERTY(VisibleAnywhere, Category = "Client Info")
	FLinearColor AvatarColor = {1.0f, 1.0f, 1.0f, 1.0f};

	/** Holds the string representation of the desktop actor class to be used as the avatar for a representation of a client */
	UPROPERTY(VisibleAnywhere, Category = "Client Info")
	FString DesktopAvatarActorClass;

	/** Holds the string representation of the VR actor class to be used as the avatar for a representation of a client */
	UPROPERTY(VisibleAnywhere, Category = "Client Info")
	FString VRAvatarActorClass;

	/** Holds an array of tags that can be used for grouping and categorizing. */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Client Info")
	TArray<FName> Tags;

	/** True if this instance was built with editor-data */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	bool bHasEditorData = true;

	/** True if this platform requires cooked data */
	UPROPERTY(VisibleAnywhere, Category="Client Info")
	bool bRequiresCookedData = false;

	CONCERT_API bool operator==(const FConcertClientInfo& Other) const;
	CONCERT_API bool operator!=(const FConcertClientInfo& Other) const;
};

/** Holds information on session client */
USTRUCT()
struct FConcertSessionClientInfo
{
	GENERATED_BODY()

	/** Create a user friendly display string for using in things such as tooltips. */
	CONCERT_API FText ToDisplayString() const;

	UPROPERTY()
	FGuid ClientEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Client Info")
	FConcertClientInfo ClientInfo;
};

/** Holds info on a session */
USTRUCT()
struct FConcertSessionInfo
{
	GENERATED_BODY()

	/** Create a user friendly display string for using in things such as tooltips. */
	CONCERT_API FText ToDisplayString() const;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FGuid ServerInstanceId;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FGuid ServerEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FGuid OwnerInstanceId;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FGuid SessionId;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FString SessionName;

	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FString OwnerUserName;

	UPROPERTY(VisibleAnywhere, Category = "Session Info")
	FString OwnerDeviceName;

	/** Settings pertaining to project, change list number etc */
	UPROPERTY(VisibleAnywhere, Category="Session Info")
	FConcertSessionSettings Settings;

	/** Version information for this session. This is set during creation, and updated each time the session is restored */
	UPROPERTY()
	TArray<FConcertSessionVersionInfo> VersionInfos;

	/** Current state of the session used to determine joinability by clients. */
	UPROPERTY()
	EConcertSessionState State = EConcertSessionState::Normal;

	FDateTime GetLastModified() const { return FDateTime(LastModifiedTicks); }
	void SetLastModified(const FDateTime& Value) { LastModifiedTicks = Value.GetTicks(); }
	void SetLastModifiedToNow() { SetLastModified(FDateTime::Now()); }

	UPROPERTY()
	FConcertSessionFileShare FileSharing;

private:
	
	/**
	 * The last time the session directory was modified in local time.
	 * 
	 * Stored in ticks instead of FDateTime because FDateTime is not serialized properly by FStructSerializer::Serialize and FStructDeserializer::Deserialize.
	 * These functions are used for packing data when sent across the network. The issue is that in UObject/NoExportTypes.h FDateTime does not expose Ticks as UProperty;
	 * doing this would be the 'proper' fix but it may cause instability this late into 5.1 dev.
	 */
	UPROPERTY(VisibleAnywhere, Category="Session Info")
	int64 LastModifiedTicks {};
};

/** Holds filter rules used when migrating session data */
USTRUCT()
struct FConcertSessionFilter
{
	GENERATED_BODY()

	/**
	 * Return true if the given activity ID passes the ID tests of this filter.
	 * @note This function only tests the ID conditions, not any data specific checks like bOnlyLiveData and bIncludeIgnoredActivities.
	 */
	CONCERT_API bool ActivityIdPassesFilter(const int64 InActivityId) const;

	/** The lower-bound (inclusive) of activity IDs to include (unless explicitly excluded via ActivityIdsToExclude) */
	UPROPERTY()
	int64 ActivityIdLowerBound = 1;

	/** The upper-bound (inclusive) of activity IDs to include (unless explicitly excluded via ActivityIdsToExclude) */
	UPROPERTY()
	int64 ActivityIdUpperBound = MAX_int64;

	/** Activity IDs to explicitly exclude, even if inside of the bounded-range specified above */
	UPROPERTY()
	TArray<int64> ActivityIdsToExclude;

	/** Activity IDs to explicitly include, even if outside of the bounded-range specified above (takes precedence over ActivityIdsToExclude) */
	UPROPERTY()
	TArray<int64> ActivityIdsToInclude;

	/** True if only live data should be included (live transactions and head package revisions) */
	UPROPERTY()
	bool bOnlyLiveData = false;

	/** True to export the activity summaries without the package/transaction data to look at the log rather than replaying the activities. */
	UPROPERTY()
	bool bMetaDataOnly = false;

	/** True to include ignored activities */
	UPROPERTY()
	bool bIncludeIgnoredActivities = false;
};

USTRUCT()
struct FConcertByteArray
{
	GENERATED_BODY()

	bool Serialize(FArchive& Ar)
	{
		int32 Num = Bytes.Num();
		Ar << Num;
		if (Ar.IsLoading())
		{
			Bytes.AddUninitialized(Num);
		}
		Ar.Serialize(Bytes.GetData(), Num);
		return true;
	}

	UPROPERTY()
	TArray<uint8> Bytes;
};

template<>
struct TStructOpsTypeTraits<FConcertByteArray> : public TStructOpsTypeTraitsBase2<FConcertByteArray>
{
	enum
	{
		WithSerializer = true,
	};
};


USTRUCT()
struct FConcertSessionSerializedPayload
{
	GENERATED_BODY()

	FConcertSessionSerializedPayload() = default;

	FConcertSessionSerializedPayload( EConcertPayloadSerializationMethod SerializeMethod )
		: SerializationMethod(SerializeMethod)
	{
	}

	/** Initialize this payload from the given data */
	CONCERT_API bool SetPayload(const FStructOnScope& InPayload, EConcertPayloadCompressionType CompressionType = EConcertPayloadCompressionType::Heuristic);
	CONCERT_API bool SetPayload(const UScriptStruct* InPayloadType, const void* InPayloadData, EConcertPayloadCompressionType CompressionType = EConcertPayloadCompressionType::Heuristic);

	template <typename T>
	bool SetTypedPayload(const T& InPayloadData, EConcertPayloadCompressionType CompressType = EConcertPayloadCompressionType::Heuristic)
	{
		return SetPayload(TBaseStructure<T>::Get(), &InPayloadData, CompressType);
	}

	/** Extract the payload into an in-memory instance */
	CONCERT_API bool GetPayload(FStructOnScope& OutPayload) const;
	CONCERT_API bool GetPayload(const UScriptStruct* InPayloadType, void* InOutPayloadData) const;

	CONCERT_API bool IsTypeChildOf(const UScriptStruct* InPayloadType) const;

	template<typename T>
	bool IsTypeChildOf() const
	{
		return IsTypeChildOf(TBaseStructure<T>::Get());
	}

	template <typename T>
	bool GetTypedPayload(T& OutPayloadData) const
	{
		return GetPayload(TBaseStructure<T>::Get(), &OutPayloadData);
	}

	/** Indicates if the stored payload has been compressed with any compression algorithm. */
	bool PayloadIsCompressed() const
	{
		return UE::Concert::Compression::DataIsCompressed(PayloadCompressionMethod);
	}

	/** The typename of the user-defined payload. */
	UPROPERTY(VisibleAnywhere, Category="Payload")
	FName PayloadTypeName;

	/** Specifies the serialization method used to pack the data */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	EConcertPayloadSerializationMethod SerializationMethod = EConcertPayloadSerializationMethod::Standard;

	/** Indicates how the serialized payload has been compressed. */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	EConcertCompressionMethod PayloadCompressionMethod = EConcertCompressionMethod::Uncompressed;

	/** Indicates how the serialized payload has been compressed. */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	EConcertCompressionBias PayloadCompressionBias = EConcertCompressionBias::None;
    
	/** The uncompressed size of the user-defined payload data. */
	UPROPERTY(VisibleAnywhere, Category="Payload")
	int32 PayloadSize = 0;

	/** The data of the user-defined payload (potentially stored as compressed binary for compact transfer). */
	UPROPERTY()
	FConcertByteArray PayloadBytes;
};
