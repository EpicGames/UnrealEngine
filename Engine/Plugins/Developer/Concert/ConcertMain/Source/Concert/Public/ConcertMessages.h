// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"

#include "ConcertTransportMessages.h"
#include "ConcertMessageData.h"
#include "ConcertMessages.generated.h"

/** Connection status for Concert client sessions */
UENUM()
enum class EConcertConnectionStatus : uint8
{
	/** Currently establishing connection to the server session */
	Connecting,
	/** Connection established and alive */
	Connected,
	/** Currently severing connection to the server session gracefully */
	Disconnecting,
	/** Disconnected */
	Disconnected,
};

/** Connection Result for Concert client session */
UENUM()
enum class EConcertConnectionResult : uint8
{
	/** No result yet */
	None,
	/** Server has accepted connection */
	ConnectionAccepted,
	/** Server has refused the connection session messages beside other connection request are ignored */
	ConnectionRefused,
	/** Server already accepted connection */
	AlreadyConnected
};

/** Status for Concert session clients */
UENUM()
enum class EConcertClientStatus : uint8
{
	/** Client connected */
	Connected,
	/** Client disconnected */
	Disconnected,
	/** Client state updated */
	Updated,

	/** Not an actual state - add new entries above. */
	Count
};

UENUM()
enum class EConcertSendReceiveState : uint8
{
	/** Full send / receive */
	Default,

	/** Sender only. Local transaction events will be sent to server but receiving data from clients has been suspended */
	SendOnly,

	/** Receive only. Updates will be received from clients but local transaction events will be deferred until state has been set to default or send. */
	ReceiveOnly
};

/** Response codes for a session custom request */
UENUM()
enum class EConcertSessionResponseCode : uint8
{
	/** The request data was valid. A response was generated. */
	Success,
	/** The request data was valid, but the request failed. A response was generated. */
	Failed,
	/** The request data was invalid. No response was generated. */
	InvalidRequest,
};

/** Response code returned when trying to mount a session repository on the server. */
UENUM()
enum class EConcertSessionRepositoryMountResponseCode : uint8
{
	/** No response code yet. */
	None,
	/** The repository was mounted on the invoked server. */
	Mounted,
	/** The repository is already mounted by another server instance. */
	AlreadyMounted,
	/** The repository ID could not be found in the server list. */
	NotFound,
};


USTRUCT()
struct FConcertAdmin_DiscoverServersEvent : public FConcertEndpointDiscoveryEvent
{
	GENERATED_BODY()

	/** The required role of the server (eg, MultiUser, DisasterRecovery, etc) */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString RequiredRole;

	/** The required version of the server (eg, 4.22, 4.23, etc) */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString RequiredVersion;

	/** If a server was configured to restrict access to specific client(s), it will search for this key in its list of authorized keys.*/
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString ClientAuthenticationKey;
};

USTRUCT()
struct FConcertAdmin_ServerDiscoveredEvent : public FConcertEndpointDiscoveryEvent
{
	GENERATED_BODY()

	/** Server designated name */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString ServerName;

	/** Basic information about the server instance */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertInstanceInfo InstanceInfo;

	/** Contains information on the server settings */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	EConcertServerFlags ServerFlags = EConcertServerFlags::None;
};

/** Contains information about a session repository. */
USTRUCT()
struct FConcertSessionRepositoryInfo
{
	GENERATED_BODY()

	/** The repository ID.*/
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid RepositoryId;

	/** The mounted state of this repository. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	bool bMounted = false;
};

/**
 * Mount a session repository used to store session files.
 */
USTRUCT()
struct FConcertAdmin_MountSessionRepositoryRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The repository unique Id. Must be valid. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid RepositoryId;

	/** The repository root dir (absolute path) where the repository should be found/created. If empty, the server will use its default one. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString RepositoryRootDir;

	/** Whether this repository is set as the default one used by server to store new sessions. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	bool bAsServerDefault = false;

	/** Whether the repository should be created if it was not found. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	bool bCreateIfNotExist = false;
};

USTRUCT()
struct FConcertAdmin_MountSessionRepositoryResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	EConcertSessionRepositoryMountResponseCode MountStatus = EConcertSessionRepositoryMountResponseCode::None;
};

/** Returns the list of repositories known of the server. */
USTRUCT()
struct FConcertAdmin_GetSessionRepositoriesRequest : public FConcertRequestData
{
	GENERATED_BODY()
};

USTRUCT()
struct FConcertAdmin_GetSessionRepositoriesResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<FConcertSessionRepositoryInfo> SessionRepositories;
};

/**
 * Drop one or more session repositories from the server, deleting all the contained files.
 */
USTRUCT()
struct FConcertAdmin_DropSessionRepositoriesRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The list of repository IDs to drop. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<FGuid> RepositoryIds;
};

USTRUCT()
struct FConcertAdmin_DropSessionRepositoriesResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The list of repository IDs successfully dropped (not found == dropped). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<FGuid> DroppedRepositoryIds;
};

USTRUCT()
struct FConcertAdmin_GetAllSessionsRequest : public FConcertRequestData
{
	GENERATED_BODY()
};

USTRUCT()
struct FConcertAdmin_GetAllSessionsResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionInfo> LiveSessions;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionInfo> ArchivedSessions;
};

USTRUCT()
struct FConcertAdmin_GetLiveSessionsRequest : public FConcertRequestData
{
	GENERATED_BODY()
};

USTRUCT()
struct FConcertAdmin_GetArchivedSessionsRequest : public FConcertRequestData
{
	GENERATED_BODY()
};

USTRUCT()
struct FConcertAdmin_GetSessionsResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionInfo> Sessions;
};

USTRUCT()
struct FConcertAdmin_CreateSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FString SessionName;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertClientInfo OwnerClientInfo;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionSettings SessionSettings;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionVersionInfo VersionInfo;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionFileShare FileShare;
};

USTRUCT()
struct FConcertAdmin_FindSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionId;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertClientInfo OwnerClientInfo;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionSettings SessionSettings;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionVersionInfo VersionInfo;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionFileShare FileShare;
};

/** Used to copy a live session or restore an archived one. */
USTRUCT()
struct FConcertAdmin_CopySessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The ID of the session to copy or restore. It can be a live or an archived session unless bRestoreOnly is true, in which case, the session to copy must be an archived one. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionId;

	/** The name of the session to create. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FString SessionName;

	/** Information about the owner of the copied session. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertClientInfo OwnerClientInfo;

	/** Settings to apply to the copied session. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionSettings SessionSettings;

	/** Version information of the client requesting the copy. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionVersionInfo VersionInfo;

	/** The filter controlling which activities from the session should be copied over. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionFilter SessionFilter;

	/** True to constrain the session to copy to be an archive (implying a restore operation). */
	bool bRestoreOnly = false;
};

USTRUCT()
struct FConcertAdmin_SessionInfoResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionInfo SessionInfo;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FConcertSessionFileShare FileShare;
};

/** Create an archived copy of a live session. */
USTRUCT()
struct FConcertAdmin_ArchiveSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The ID of the session to archive (must be a live session). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The override for the archive. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString ArchiveNameOverride;

	/** The caller user name. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString UserName;

	/** The caller device name. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString DeviceName;

	/** The filter controlling which activities from the session should be archived. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FConcertSessionFilter SessionFilter;
};

USTRUCT()
struct FConcertAdmin_ArchiveSessionResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The ID of the session that was requested to be archived. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The name of the session that was requested to be archived. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString SessionName;

	/** The ID of the new archived session (on success). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid ArchiveId;

	/** The name of the new archived session (on success). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString ArchiveName;
};

/** Rename a session. */
USTRUCT()
struct FConcertAdmin_RenameSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The ID of the session to rename. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The new session name. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString NewName;

	// For now only the user name and device name of the client is used to id it as the owner of a session
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString UserName;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString DeviceName;
};

USTRUCT()
struct FConcertAdmin_RenameSessionResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The ID of the session that was requested to be renamed. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The old session name (if the session exist). */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString OldName;
};


/** Delete an archived or live session. */
USTRUCT()
struct FConcertAdmin_DeleteSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The ID of the session to delete. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	//For now only the user name and device name of the client is used to id it as the owner of a session
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString UserName;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString DeviceName;
};

USTRUCT()
struct FConcertAdmin_DeleteSessionResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The ID of the session that was requested to be deleted. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The name of the session that was was requested to be deleted. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString SessionName;
};

UENUM()
enum class EBatchSessionDeletionFlags
{
	/** Any error in the request will make the entire operation fail. */
	Strict,

	/**
	 * If the request contains sessions that is not owned by the requesting client, skip them.
	 * Skipped sessions are added to FConcertAdmin_BatchDeleteSessionResponse::NotOwnedByClient.
	 * This avoids clients having to request ownership info before ensuring an atomic and consistent operation.
	 */
	SkipForbiddenSessions = 0x01
};
ENUM_CLASS_FLAGS(EBatchSessionDeletionFlags)

/** Deletes several archived and/or live sessions. */
USTRUCT()
struct FConcertAdmin_BatchDeleteSessionRequest : public FConcertRequestData
{
	GENERATED_BODY()

	/** The ID of the session to delete. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TSet<FGuid> SessionIds;

	/** How the operation input is supposed to be interpreted */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	EBatchSessionDeletionFlags Flags = EBatchSessionDeletionFlags::Strict;

	//For now only the user name and device name of the client is used to id it as the owner of a session
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString UserName;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString DeviceName;
};

USTRUCT()
struct FDeletedSessionInfo
{
	GENERATED_BODY()
	
	/** The ID of the session that was requested to be deleted. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FGuid SessionId;

	/** The name of the session that was was requested to be deleted. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	FString SessionName;
};

/** Answer by server */
USTRUCT()
struct FConcertAdmin_BatchDeleteSessionResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The IDs of the session that were requested to be deleted. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<FDeletedSessionInfo> DeletedItems;

	/** Contains the sessions that were skipped if FConcertAdmin_BatchDeleteSessionRequest::Flags has the SkipForbiddenSessions flag set. */
	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<FDeletedSessionInfo> NotOwnedByClient;
};

USTRUCT()
struct FConcertAdmin_GetSessionClientsRequest : public FConcertRequestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionId;
};

USTRUCT()
struct FConcertAdmin_GetSessionClientsResponse : public FConcertResponseData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionClientInfo> SessionClients;
};

USTRUCT()
struct FConcertAdmin_GetSessionActivitiesRequest : public FConcertRequestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionId;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	int64 FromActivityId = 1;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	int64 ActivityCount = 1024;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	bool bIncludeDetails = false;
};

USTRUCT()
struct FConcertAdmin_GetSessionActivitiesResponse : public FConcertResponseData
{
	GENERATED_BODY()

	/** The list of activities. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionSerializedPayload> Activities;

	/** Maps each activity endpoint to its client info. */
	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TMap<FGuid, FConcertClientInfo> EndpointClientInfoMap;
};

USTRUCT()
struct FConcertSession_DiscoverAndJoinSessionEvent : public FConcertEndpointDiscoveryEvent
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionServerEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertClientInfo ClientInfo;
};

USTRUCT()
struct FConcertSession_JoinSessionResultEvent : public FConcertEndpointDiscoveryEvent
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionServerEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	EConcertConnectionResult ConnectionResult = EConcertConnectionResult::None;

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionClientInfo> SessionClients;
};

USTRUCT()
struct FConcertSession_LeaveSessionEvent : public FConcertEventData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FGuid SessionServerEndpointId;
};

USTRUCT()
struct FConcertSession_UpdateClientInfoEvent : public FConcertEventData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FConcertSessionClientInfo SessionClient;
};

USTRUCT()
struct FConcertSession_ClientListUpdatedEvent : public FConcertEventData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	TArray<FConcertSessionClientInfo> SessionClients;
};

USTRUCT()
struct FConcertSession_SessionRenamedEvent : public FConcertEventData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Concert Message")
	FString NewName;
};

USTRUCT()
struct FConcertSession_CustomEvent : public FConcertEventData
{
	GENERATED_BODY()

	virtual bool IsSafeToHandle() const override
	{
		return !(UE::IsSavingPackage() || IsGarbageCollecting());
	}

	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FGuid SourceEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	TArray<FGuid> DestinationEndpointIds;

	/** The serialized payload that we're hosting. */
	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FConcertSessionSerializedPayload SerializedPayload;
};

USTRUCT()
struct FConcertSession_CustomRequest : public FConcertRequestData
{
	GENERATED_BODY()

	virtual bool IsSafeToHandle() const override
	{
		return !(UE::IsSavingPackage() || IsGarbageCollecting());
	}

	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FGuid SourceEndpointId;

	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FGuid DestinationEndpointId;

	/** The serialized payload that we're hosting. */
	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FConcertSessionSerializedPayload SerializedPayload;
};

USTRUCT()
struct FConcertSession_CustomResponse : public FConcertResponseData
{
	GENERATED_BODY()

	virtual bool IsSafeToHandle() const override
	{
		return !(UE::IsSavingPackage() || IsGarbageCollecting());
	}

	/** Set the internal Concert response code from the custom response code from the request handler */
	void SetResponseCode(const EConcertSessionResponseCode InResponseCode)
	{
		switch (InResponseCode)
		{
		case EConcertSessionResponseCode::Success:
			ResponseCode = EConcertResponseCode::Success;
			break;
		case EConcertSessionResponseCode::Failed:
			ResponseCode = EConcertResponseCode::Failed;
			break;
		case EConcertSessionResponseCode::InvalidRequest:
			ResponseCode = EConcertResponseCode::InvalidRequest;
			break;
		default:
			checkf(false, TEXT("Unknown EConcertSessionResponseCode!"));
			break;
		}
	}

	/** The serialized payload that we're hosting. */
	UPROPERTY(VisibleAnywhere, Category="Concert Custom Message")
	FConcertSessionSerializedPayload SerializedPayload;
};
