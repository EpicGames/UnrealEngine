// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"

class IConcertServer;
class IConcertServerSession;

// Most events from IConcertServerEventSink are mirroed here
namespace ConcertServerEvents
{
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLiveSessionCreated, bool /*bSuccess*/, const IConcertServer& /*InServer*/, TSharedRef<IConcertServerSession> /*InLiveSession*/);
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnArchivedSessionCreated, bool /*bSuccess*/, const IConcertServer& /*InServer*/, const FString& /*InArchivedSessionRoot*/, const FConcertSessionInfo& /*InArchivedSessionInfo*/);
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLiveSessionDestroyed, const IConcertServer& /*InServer*/, TSharedRef<IConcertServerSession> /*InLiveSession*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnArchivedSessionDestroyed, const IConcertServer& /*InServer*/, const FGuid& /*InArchivedSessionId*/);
	DECLARE_MULTICAST_DELEGATE_SixParams(FArchiveSession_WithSession, bool /*bSuccess*/, const IConcertServer& /*InServer*/, TSharedRef<IConcertServerSession> /*InLiveSession*/, const FString& /*InArchivedSessionRoot*/, const FConcertSessionInfo& /*InArchivedSessionInfo*/, const FConcertSessionFilter& /*InSessionFilter*/);
	DECLARE_MULTICAST_DELEGATE_SixParams(FArchiveSession_WithWorkingDir, bool /*bSuccess*/, const IConcertServer& /*InServer*/, const FString& /*InLiveSessionWorkingDir*/, const FString& /*InArchivedSessionRoot*/, const FConcertSessionInfo& /*InArchivedSessionInfo*/, const FConcertSessionFilter& /*InSessionFilter*/);

	DECLARE_MULTICAST_DELEGATE_FiveParams(FCopySession, bool /*bSuccess*/, const IConcertServer& /*InServer*/, TSharedRef<IConcertServerSession> /*InLiveSession*/, const FString& /*NewSessionRoot*/, const FConcertSessionFilter& /*InSessionFilter*/);
	DECLARE_MULTICAST_DELEGATE_SixParams(FExportSession, bool/*bSuccess*/, const IConcertServer& /*InServer*/, const FGuid& /*InSessionId*/, const FString& /*DestDir*/, const FConcertSessionFilter& /*InSessionFilter*/, bool /*bAnonymizeData*/);
	DECLARE_MULTICAST_DELEGATE_SixParams(FRestoreSession, bool/*bSuccess*/, const IConcertServer& /*InServer*/, const FGuid& /*InArchivedSessionId*/, const FString& /*InLiveSessionRoot*/, const FConcertSessionInfo& /*InLiveSessionInfo*/, const FConcertSessionFilter& /*InSessionFilter*/);
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLiveSessionRenamed, const IConcertServer& /*InServer*/, TSharedRef<IConcertServerSession> /*InLiveSession*/);
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnArchivedSessionRenamed, const IConcertServer& /*InServer*/, const FString& /*InArchivedSessionRoot*/, const FConcertSessionInfo& /*InArchivedSessionInfo*/);

	CONCERT_API FOnLiveSessionCreated& OnLiveSessionCreated();
	CONCERT_API FOnArchivedSessionCreated& OnArchivedSessionCreated();
	CONCERT_API FOnLiveSessionDestroyed& OnLiveSessionDestroyed();
	CONCERT_API FOnArchivedSessionDestroyed& OnArchivedSessionDestroyed();
	CONCERT_API FArchiveSession_WithSession& ArchiveSession_WithSession();
	CONCERT_API FArchiveSession_WithWorkingDir& ArchiveSession_WithWorkingDir();
	CONCERT_API FCopySession& CopySession();
	CONCERT_API FExportSession& ExportSession();
	CONCERT_API FRestoreSession& RestoreSession();
	CONCERT_API FOnLiveSessionRenamed& OnLiveSessionRenamed();
	CONCERT_API FOnArchivedSessionRenamed& OnArchivedSessionRenamed();
}