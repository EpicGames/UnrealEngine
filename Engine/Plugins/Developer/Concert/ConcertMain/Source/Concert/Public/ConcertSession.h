// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertSession.h"
#include "ConcertMessages.h"

#define UE_API CONCERT_API

/**
 * Common implementation for Concert Client and Server sessions (@see IConcertSession for the API description).
 * @note	This doesn't inherit from any session interface, but does implement some of their API with a "Common" prefix on the function names.
 *			Interface implementations can also inherit from this common impl and then call the "Common" functions from the interface overrides.
 */
class FConcertSessionCommonImpl
{
public:
	UE_API explicit FConcertSessionCommonImpl(const FConcertSessionInfo& InSessionInfo);

	UE_API void CommonStartup();
	UE_API void CommonShutdown();

	const FGuid& CommonGetId() const
	{
		return SessionInfo.SessionId;
	}

	const FString& CommonGetName() const
	{
		return SessionInfo.SessionName;
	}

	void CommonSetName(const FString& NewName)
	{
		SessionInfo.SessionName = NewName;
	}

	const FConcertSessionInfo& CommonGetSessionInfo() const
	{
		return SessionInfo;
	}

	UE_API TArray<FGuid> CommonGetSessionClientEndpointIds() const;
	UE_API TArray<FConcertSessionClientInfo> CommonGetSessionClients() const;
	UE_API bool CommonFindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const;

	UE_API FConcertScratchpadRef CommonGetScratchpad() const;
	UE_API FConcertScratchpadPtr CommonGetClientScratchpad(const FGuid& ClientEndpointId) const;

	UE_API FDelegateHandle CommonRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler);
	UE_API void CommonUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle);
	UE_API void CommonUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler);
	UE_API void CommonClearCustomEventHandler(const FName& EventMessageType);
	UE_API void CommonHandleCustomEvent(const FConcertMessageContext& Context);

	UE_API void CommonRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler);
	UE_API void CommonUnregisterCustomRequestHandler(const FName& RequestMessageType);
	UE_API TFuture<FConcertSession_CustomResponse> CommonHandleCustomRequest(const FConcertMessageContext& Context);

	static UE_API bool CommonBuildCustomEvent(const UScriptStruct* EventType, const void* EventData, const FGuid& SourceEndpointId, const TArray<FGuid>& DestinationEndpointIds, FConcertSession_CustomEvent& OutCustomEvent);
	static UE_API bool CommonBuildCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& SourceEndpointId, const FGuid& DestinationEndpointId, FConcertSession_CustomRequest& OutCustomRequest);
	static UE_API void CommonHandleCustomResponse(const FConcertSession_CustomResponse& Response, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler);

protected:
	/** Information about this session */
	FConcertSessionInfo SessionInfo;

	/** The scratchpad for this session */
	FConcertScratchpadPtr Scratchpad;

	/** Map of clients connected to this session (excluding us if we're a client session) */
	struct FSessionClient
	{
		FConcertSessionClientInfo ClientInfo;
		FConcertScratchpadPtr Scratchpad;
	};
	TMap<FGuid, FSessionClient> SessionClients;

	/** Map of custom event handlers for this session */
	TMap<FName, TArray<TSharedPtr<IConcertSessionCustomEventHandler>>> CustomEventHandlers;

	/** Map of custom request handlers for this session */
	TMap<FName, TSharedPtr<IConcertSessionCustomRequestHandler>> CustomRequestHandlers;
};

#undef UE_API
