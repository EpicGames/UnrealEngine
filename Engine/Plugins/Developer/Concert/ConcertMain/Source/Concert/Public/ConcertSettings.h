// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "ConcertTransportSettings.h"
#include "ConcertVersion.h"

#include "ConcertSettings.generated.h"

namespace ConcertSettingsUtils
{

/** Returns an error messages if the user display name is invalid, otherwise, returns an empty text. */
FText CONCERT_API ValidateDisplayName(const FString& Name);

/** Returns an error messages if the specified session name is invalid, otherwise, returns an empty text. */
FText CONCERT_API ValidateSessionName(const FString& Name);

/** Returns an error messages if the specified path name is invalid, otherwise, returns an empty text. */
FText CONCERT_API ValidateFileSharingPath(const FString& Path);

}
USTRUCT()
struct FConcertSessionSettings
{
	GENERATED_BODY()

	void Initialize()
	{
		ProjectName = FApp::GetProjectName();
		// TODO: BaseRevision should have a robust way to know which content version a project is on, as we currently check this using the current build version (see EngineVersion in FConcertSessionVersionInfo), which works for UGS but isn't reliable for public binary builds
	}

	bool ValidateRequirements(const FConcertSessionSettings& Other, FText* OutFailureReason = nullptr) const
	{
		if (ProjectName != Other.ProjectName)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(NSLOCTEXT("ConcertMain", "Error_InvalidProjectNameFmt", "Invalid project name (expected '{0}', got '{1}')"), FText::AsCultureInvariant(ProjectName), FText::AsCultureInvariant(Other.ProjectName));
			}
			return false;
		}

		if (BaseRevision != Other.BaseRevision)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(NSLOCTEXT("ConcertMain", "Error_InvalidBaseRevisionFmt", "Invalid base revision (expected '{0}', got '{1}')"), BaseRevision, Other.BaseRevision);
			}
			return false;
		}

		return true;
	}

	/**
	 * Name of the project of the session.
	 * Can be specified on the server cmd with `-CONCERTPROJECT=`
	 */
	UPROPERTY(config, VisibleAnywhere, Category="Session Settings")
	FString ProjectName;

	/**
	 * Base Revision the session was created at.
	 * Can be specified on the server cmd with `-CONCERTREVISION=`
	 */
	UPROPERTY(config, VisibleAnywhere, Category="Session Settings")
	uint32 BaseRevision = 0;

	/**
	 * Override the default name chosen when archiving this session.
	 * Can be specified on the server cmd with `-CONCERTSAVESESSIONAS=`
	 */
	UPROPERTY(config, VisibleAnywhere, Category="Session Settings")
	FString ArchiveNameOverride;

	// TODO: private session, password, etc etc,
};
