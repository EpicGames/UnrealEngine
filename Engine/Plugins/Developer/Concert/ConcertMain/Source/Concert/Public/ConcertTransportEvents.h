// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertTransportEvents.generated.h"

class IConcertClient;
class IConcertServer;

UENUM()
enum class EConcertLogMessageAction : uint8
{
	None,
	Send,
	Publish,
	Receive,
	Queue,
	Discard,
	Duplicate,
	TimeOut,
	Process,
	EndpointDiscovery,
	EndpointTimeOut,
	EndpointClosure,
};

USTRUCT()
struct FConcertLog
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 Frame = 0;

	UPROPERTY()
	FGuid MessageId;

	UPROPERTY()
	uint16 MessageOrderIndex  = 0;

	UPROPERTY()
	uint16 ChannelId = 0;

	UPROPERTY()
	FDateTime Timestamp = {0};

	UPROPERTY()
	EConcertLogMessageAction MessageAction = EConcertLogMessageAction::None;

	UPROPERTY()
	FName MessageTypeName;

	UPROPERTY()
	FGuid OriginEndpointId;

	UPROPERTY()
	FGuid DestinationEndpointId;

	UPROPERTY()
	FName CustomPayloadTypename;

	UPROPERTY()
	int32 CustomPayloadUncompressedByteSize = 0;

	UPROPERTY()
	FString StringPayload;

	UPROPERTY(Transient)
	FConcertSessionSerializedPayload SerializedPayload;
};

namespace ConcertTransportEvents
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FConcertTransportLoggingEnabledChanged, bool /*bIsEnabled*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FConcertClientLogEvent, const IConcertClient&, const FConcertLog&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FConcertServerLogEvent, const IConcertServer&, const FConcertLog&);

	CONCERT_API bool IsLoggingEnabled();
	CONCERT_API void SetLoggingEnabled(bool bEnabled);

	CONCERT_API FConcertTransportLoggingEnabledChanged& OnConcertTransportLoggingEnabledChangedEvent();
	/** Use this when your local instance is a client. All client-generated logs are emitted by this event. */
	CONCERT_API FConcertClientLogEvent& OnConcertClientLogEvent();
	/** Use this when your local instance is a server. All server-generated logs are emitted by this event. */
	CONCERT_API FConcertServerLogEvent& OnConcertServerLogEvent();
}

