// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertTransportLogger.h"

class IConcertServerSession;
class IConcertServer;
struct FConcertSessionClientInfo;
struct FConcertLog;
using FLogListener = TFunction<void(const FConcertLog&)>;

namespace ConcertUtil
{

/** Delete a directory tree via a move and delete */
CONCERT_API bool DeleteDirectoryTree(const TCHAR* InDirectoryToDelete, const TCHAR* InMoveToDirBeforeDelete = nullptr);

/** Copy the specified data size from a source archive into a destination archive. */
CONCERT_API bool Copy(FArchive& DstAr, FArchive& SrcAr, int64 Size);

/** Turn on verbose logging for all loggers (including console loggers). */
CONCERT_API void SetVerboseLogging(bool bInState);

/** Factory function for use with FConcertTransportLoggerFactory */
CONCERT_API IConcertTransportLoggerRef CreateLogger(const FConcertEndpointContext& InOwnerContext, FLogListener LogListenerFunc);

} // namespace ConcertUtil
