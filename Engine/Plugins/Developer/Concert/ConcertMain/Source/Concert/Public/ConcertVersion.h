// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertVersion.generated.h"

class FEngineVersion;
struct FCustomVersion;

/** Modes that can be used when validating Concert version information */
enum EConcertVersionValidationMode : uint8
{
	/** Check that the version information is identical */
	Identical,
	/** Check that the version information is compatible (other version is equal or newer than the current version) */
	Compatible,
	/** Check that the major and minor version are identical but allow compatible patch versions.*/
	PatchCompatible
};

/** Holds file version information */
USTRUCT()
struct FConcertFileVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the compiled in data */
	CONCERT_API void Initialize();

	/** Validate this version info against another */
	CONCERT_API bool Validate(const FConcertFileVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason = nullptr) const;

	/* UE4 File version */
	UPROPERTY()
	int32 FileVersion = 0; // Ideally we would rename this FileVersionUE4

	/* UE5 File version */
	UPROPERTY()
	int32 FileVersionUE5 = 0;

	/* Licensee file version */
	UPROPERTY()
	int32 FileVersionLicensee = 0;
};

/** Holds engine version information */
USTRUCT()
struct FConcertEngineVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the given version */
	CONCERT_API void Initialize(const FEngineVersion& InVersion);

	/** Validate this version info against another */
	CONCERT_API bool Validate(const FConcertEngineVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason = nullptr) const;

	/** Major version number */
	UPROPERTY()
	uint16 Major = 0;

	/** Minor version number */
	UPROPERTY()
	uint16 Minor = 0;

	/** Patch version number */
	UPROPERTY()
	uint16 Patch = 0;

	/** Changelist number. This is used to arbitrate when Major/Minor/Patch version numbers match */
	UPROPERTY()
	uint32 Changelist = 0;
};

/** Holds custom version information */
USTRUCT()
struct FConcertCustomVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the given version */
	CONCERT_API void Initialize(const FCustomVersion& InVersion);

	/** Validate this version info against another */
	CONCERT_API bool Validate(const FConcertCustomVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason = nullptr) const;

	/** Friendly name of the version */
	UPROPERTY()
	FName FriendlyName;

	/** Unique custom key */
	UPROPERTY()
	FGuid Key;

	/** Custom version */
	UPROPERTY()
	int32 Version = 0;
};

/** Holds version information for a session */
USTRUCT()
struct FConcertSessionVersionInfo
{
	GENERATED_BODY()

	/** Initialize this version info from the compiled in data */
	CONCERT_API void Initialize(bool bSupportMixedBuildTypes);

	/** Validate this version info against another */
	CONCERT_API bool Validate(const FConcertSessionVersionInfo& InOther, const EConcertVersionValidationMode InValidationMode, FText* OutFailureReason = nullptr) const;

	/** Return the version information in string represenation. */
	CONCERT_API FText AsText() const;

	/** File version info */
	UPROPERTY()
	FConcertFileVersionInfo FileVersion;

	/** Engine version info */
	UPROPERTY()
	FConcertEngineVersionInfo EngineVersion;

	/** Custom version info */
	UPROPERTY()
	TArray<FConcertCustomVersionInfo> CustomVersions;
};
