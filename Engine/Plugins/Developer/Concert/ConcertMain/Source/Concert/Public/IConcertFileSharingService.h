// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

/**
 * Implements a file sharing service used as a side channel to ConcertTransport to exchange large files. This API is designed to
 * work hand in hand with the request/response/event layer.
 *
 * When a file needs to be exchanged between the client and the server:
 *
 * The publisher side would:
 *    - Publish a file using file sharing service. The service tracks the request and capture the file for sharing (can sent it to a server asynchronously, can be copied to a shared folder, etc.)
 *    - Notify the consumer(s) by sending a Concert request/event/response with the ID of the published file.
 *
 * The consumer(s) side would:
 *    - Receive and process the Concert request/event/response and extract the file ID.
 *    - Create a stream to read the shared file, ideally in a background task. If required, new method could be added to simplify asynchrous IO.
 *
 * * @note The lifetime of the shared files is implementation specific.
 *
 */
class IConcertFileSharingService
{
public:
	virtual ~IConcertFileSharingService() = default;

	/** Make a clone of this sharing service to be used. Ownership is transferred to the caller. Callers can choose to wrap this into a unique or shared pointer. */
	virtual IConcertFileSharingService* Clone() const = 0;

	/** Indicates if the file sharing service is enabled. */
	virtual bool IsEnabled() const = 0;

	/** Turn on/off the file sharing service. */
	virtual void SetEnabled(bool bInEnabled) = 0;

	/**
	 * Sets the desired path to use for the service. Implementors have the right to ignore the path specified.
	 *
	 * Implementors should set the enabled state based on if the path can be written. 
	 * 
	 * @param FileShareRoot to set as the root folder for content.
	 */
	virtual void SetPath(const FString& FileShareRoot) = 0;

	/**
	 * Get the assign path for file sharing root.
	 */
	virtual FString GetPath() const = 0;

	/** Deletes the specified outfile id. */
	virtual void Delete(const FString& InFileId) = 0;
	
	/**
	 * Copy the specified files in a uniquely named file stored in the service directory for sharing. The lifetime of the shared file
	 * is implementation specific.
	 * @param Pathname The file to copy and share.
	 * @param OutFileId The ID of the published file. The publisher is expected to send the ID through the Concert transport layer to the consumer(s) (request/response/event).
	 * @return True if the file was published successfully.
	 */
	virtual bool Publish(const FString& Pathname, FString& OutFileId) = 0;

	/**
	 * Copies up to Count bytes contained in the archives in a uniquely named file stored in the service directory for sharing.
	 * The functions reads the archive from its current position up to \a Count bytes. The lifetime of the shared file
	 * is implementation specific.
	 * @param SrcAr The Archive containing the bytes to share. The archive is already at the right position, not necessarily 0.
	 * @param Count The number of bytes to read from the archive.
	 * @param OutFileId The ID of the published file. The publisher is expected to send the ID through the Concert transport layer to the consumer(s) (request/response/event).
	 * @return True if the file was published successfully.
	 */
	virtual bool Publish(FArchive& SrcAr, int64 Count, FString& OutFileId) = 0;

	/**
	 * Copies all the bytes contained in the archives in a uniquely named file stored in the service directory for sharing.
	 * @param SrcAr The Archive containing the bytes to share. Expected to be at position 0.
	 * @param OutFileId The ID of the published file. The publisher is expected to send the ID through the Concert transport layer to the consumer(s) (request/response/event).
	 * @return True if the file was published successfully.
	 */
	bool Publish(FArchive& SrcAr, FString& OutFileId) { return Publish(SrcAr, SrcAr.TotalSize(), OutFileId); }

	/**
	 * Open a shared file for consumption.
	 * @param InFileId The Id of the file to read. The consumer is expected to receives this ID through the Concert transport layer (request/response/event).
	 * @return An archive (loading mode) to stream the content of the file or nullptr if the file could not be found or opened.
	 * @note The function can be used to poll until the resources is fully available. A more sophisticated API could be added to wait for a ressource if needed later.
	 */
	virtual TSharedPtr<FArchive> CreateReader(const FString& InFileId) = 0;


	/**
	 * Open a file for writing.
	 * @param OutFileId The id of the file opened for write. 
	 * @return An archive (write mode) to stream the content of the file.
	 */
	virtual TSharedPtr<FArchive> CreateWriter(FString& OutFileId) = 0;
};
