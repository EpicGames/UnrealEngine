// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"

class IConcertServer;
class IConcertServerSession;

struct FConcertSessionInfo;
struct FConcertSessionFilter;

struct FInternalLiveSessionCreationParams
{
	DECLARE_DELEGATE(FOnSessionModified)

	/** Called when the session was modified */
	FOnSessionModified OnModifiedCallback;
};

/** Interface for events that Concert server can emit */
class IConcertServerEventSink
{
public:
	/**
	 * Called to enumerate all the sessions under the given root path and retrieve their session info.
	 */
	virtual void GetSessionsFromPath(const IConcertServer& InServer, const FString& InPath, TArray<FConcertSessionInfo>& OutSessionInfos, TArray<FDateTime>* OutSessionCreationTimes = nullptr) = 0;

	/**
	 * Called after the session has been created (and before Startup has been called on it).
	 * @note This function is called for both newly created sessions and after recovering a live session during server start-up.
	 * @return true if the session creation could be completed without error, false otherwise (ex if the database fails to open).
	 */
	virtual bool OnLiveSessionCreated(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FInternalLiveSessionCreationParams& AdditionalParams) = 0;
	
	/**
	 * Called before the session is destroyed (and before Shutdown is called on it).
	 * @note Destroyed in this case means that the resources for the session should be closed/freed, but not that persistent data should be deleted from disk.
	 */
	virtual void OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) = 0;
	
	/**
	 * Called after the session has been created.
	 * @note This function is called for both newly created sessions and after recovering an archived session during server start-up.
	 * @return true if the session creation could be completed without error, false otherwise (ex if the database fails to open).
	 */
	virtual bool OnArchivedSessionCreated(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo) = 0;
	
	/**
	 * Called before the session is destroyed.
	 * @note Destroyed in this case means that the resources for the session should be closed/freed, but not that persistent data should be deleted from disk.
	 */
	virtual void OnArchivedSessionDestroyed(const IConcertServer& InServer, const FGuid& InArchivedSessionId) = 0;
	
	/**
	 * Called to migrate the data for a live session into an archived session.
	 * @note OnArchivedSessionCreated will be called if this archive was successful.
	 */
	virtual bool ArchiveSession(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter) = 0;
	
	/**
	 * Called to migrate the data of an offline live session into and offline archived session.
	 * @note This function is use at boot time to auto-archive session that were not archived at shutdown because the server crashed or was killed.
	 */
	virtual bool ArchiveSession(const IConcertServer& InServer, const FString& InLiveSessionWorkingDir, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter) = 0;

	/**
	 * Called to copy the data of a live session into another live session.
	 */
	virtual bool CopySession(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& NewSessionRoot, const FConcertSessionFilter& InSessionFilter) = 0;

	/**
	 * Called to migrate and gather the data of a live or archived session to a directory for external usage.
	 * @note As opposed to archiving a session, once exported, the session is not tracked anymore by the server and the exported files can be copied/modified while the server is running.
	 */
	virtual bool ExportSession(const IConcertServer& InServer, const FGuid& InSessionId, const FString& DestDir, const FConcertSessionFilter& InSessionFilter, bool bAnonymizeData) = 0;

	/**
	 * Called to migrate the data for an archived session into a live session.
	 * @note OnLiveSessionCreated will be called if this restoration was successful.
	 */
	virtual bool RestoreSession(const IConcertServer& InServer, const FGuid& InArchivedSessionId, const FString& InLiveSessionRoot, const FConcertSessionInfo& InLiveSessionInfo, const FConcertSessionFilter& InSessionFilter) = 0;

	/**
	 * Called to get the all activities that are not muted (i.e. don't have EConcertSyncActivityFlags::Muted flag set) for an archived or a live session without being connected to it.
	 * @note If ActivityCount is negative, the function returns the last activities (the tail) from Max(1, TotalActivityCount + ActivityCount + 1)
	 */
	virtual bool GetUnmutedSessionActivities(const IConcertServer& InServer, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& OutActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails) = 0;

	/**
	 * Called when a live session is renamed.
	 */
	virtual void OnLiveSessionRenamed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) = 0;

	/**
	 * Called when an archived session is renamed.
	 */
	virtual void OnArchivedSessionRenamed(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo) = 0;
};
