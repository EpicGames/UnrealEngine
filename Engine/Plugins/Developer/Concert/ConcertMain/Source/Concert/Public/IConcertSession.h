// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageContext.h"
#include "CoreMinimal.h"
#include "IConcertEndpoint.h"
#include "IConcertSessionHandler.h"
#include "ConcertMessages.h"
#include "Containers/ArrayBuilder.h"
#include "Scratchpad/ConcertScratchpadPtr.h"
#include "IdentifierTable/ConcertIdentifierTablePtr.h"

class IConcertSession;
class IConcertServerSession;
class IConcertClientSession;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientSessionTick, IConcertClientSession&, float);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertServerSessionTick, IConcertServerSession&, float);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientSessionConnectionChanged, IConcertClientSession&, EConcertConnectionStatus);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConcertClientSessionClientChanged, IConcertClientSession&, EConcertClientStatus, const FConcertSessionClientInfo&);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConcertServerSessionClientChanged, IConcertServerSession&, EConcertClientStatus, const FConcertSessionClientInfo&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertSessionRenamed, const FString&, const FString&);

/** Contains the FConcertClientInfo fields that can be updated. */
struct FConcertClientInfoUpdate
{
	TOptional<FString> DisplayName;
	TOptional<FLinearColor> AvatarColor;
	TOptional<FString> DesktopAvatarActorClass;
	TOptional<FString> VRAvatarActorClass;

	/** Applies the valid optional fields to the specified client info, and return true if the client info was modified, false otherwise. */
	bool ApplyTo(FConcertClientInfo& InOutInfo) const
	{
		bool bUpdated = false;
		if (DisplayName.IsSet() && DisplayName.GetValue() != InOutInfo.DisplayName)
		{
			InOutInfo.DisplayName = DisplayName.GetValue();
			bUpdated = true;
		}

		if (AvatarColor.IsSet() && AvatarColor.GetValue() != InOutInfo.AvatarColor)
		{
			InOutInfo.AvatarColor = AvatarColor.GetValue();
			bUpdated = true;
		}

		if (DesktopAvatarActorClass.IsSet() && DesktopAvatarActorClass.GetValue() != InOutInfo.DesktopAvatarActorClass)
		{
			InOutInfo.DesktopAvatarActorClass = DesktopAvatarActorClass.GetValue();
			bUpdated = true;
		}

		if (VRAvatarActorClass.IsSet() && VRAvatarActorClass.GetValue() != InOutInfo.VRAvatarActorClass)
		{
			InOutInfo.VRAvatarActorClass = VRAvatarActorClass.GetValue();
			bUpdated = true;
		}

		return bUpdated;
	}
};

class FConcertSequencedCustomEventManager;

struct FConcertSequencedCustomEvent
{
	FConcertSequencedCustomEvent() = default;
	~FConcertSequencedCustomEvent() = default;

private:
	friend class FConcertSequencedCustomEventManager;

	FConcertSequencedCustomEvent(int32 InIndex)
		: Index(InIndex)
	{
	}

	int32 Index = INDEX_NONE;
};

/** Allows users to reserve a spot in the outbound queues. */
class FConcertSequencedCustomEventManager
{
public:
	FConcertSequencedCustomEventManager()
	{
	}
	~FConcertSequencedCustomEventManager() = default;

	FConcertSequencedCustomEvent AddSequencedCustomEvent()
	{
		Events.Add(CurrentId, {});
		return FConcertSequencedCustomEvent(CurrentId++);
	}

	void RemoveSequencedEvent(const FConcertSequencedCustomEvent& EventDetails)
	{
		if (EventDetails.Index == INDEX_NONE)
		{
			return;
		}
		Events.Remove(EventDetails.Index);
	}

	bool HasEventsPending() const
	{
		return !Events.IsEmpty();
	}

	struct FPendingCustomEvent
	{
		FConcertSession_CustomEvent CustomEvent;
		EConcertMessageFlags Flags;
		TMap<FName, FString> Annotations;
		FGuid ServerEndpointId;
	};

	TOptional<FPendingCustomEvent> PopSendEvent()
	{
		TOptional<FPendingCustomEvent> ReturnVal;
		if (LastSendId < CurrentId)
		{
			TOptional<FPendingCustomEvent> *CustomEvent = Events.Find(LastSendId);
			// Find the first valid value we have stored.
			while (!CustomEvent)
			{
				LastSendId++;
				CustomEvent = Events.Find(LastSendId);
			}
			if (*CustomEvent)
			{
				// If the optional has a value then it is available for sending.
				ReturnVal = *CustomEvent;
				Events.Remove(LastSendId);
				LastSendId++;
			}
		}
		return ReturnVal;
	}

	void FillPendingSequenceEvent(const FConcertSequencedCustomEvent& InEventId, FPendingCustomEvent InEventData)
	{
		TOptional<FPendingCustomEvent>* PendingEvent = Events.Find(InEventId.Index);
		// The event data should exist and it should not have any data in the optional.
		check(PendingEvent);
		check(!(*PendingEvent));
		*PendingEvent = MoveTemp(InEventData);
	}

	bool CanSend(const FConcertSequencedCustomEvent& InEventId) const
	{
		return InEventId.Index == LastSendId;
	}

private:
	TMap<int32, TOptional<FPendingCustomEvent> > Events;

	int32 LastSendId = 0;
	int32 CurrentId = 0;
};

/** Interface for Concert sessions */
class IConcertSession
{
public:
	virtual ~IConcertSession() = default;

	virtual void Startup() = 0;

	virtual void Shutdown() = 0;

	virtual const FGuid& GetId() const = 0;

	virtual const FString& GetName() const = 0;

	virtual const FConcertSessionInfo& GetSessionInfo() const = 0;

	/** Give the working directory for this session */
	virtual FString GetSessionWorkingDirectory() const = 0;

	/** Get the list of connected client endpoint IDs */
	virtual TArray<FGuid> GetSessionClientEndpointIds() const = 0;

	/** Get the information about all connected clients */
	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const = 0;

	/** Find the the client for the specified endpoint ID */
	virtual bool FindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const = 0;

	/**
	 * Get the scratchpad associated with this concert session.
	 */
	virtual FConcertScratchpadRef GetScratchpad() const = 0;

	/**
	 * Get the scratchpad associated with the given client ID.
	 */
	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid& ClientEndpointId) const = 0;
	

	/**
	 * Register a custom event handler for this session
	 */
	template<typename EventType>
	FDelegateHandle RegisterCustomEventHandler(typename TConcertFunctionSessionCustomEventHandler<EventType>::FFuncType Func)
	{
		return InternalRegisterCustomEventHandler(EventType::StaticStruct()->GetFName(), MakeShared<TConcertFunctionSessionCustomEventHandler<EventType>>(FDelegateHandle(FDelegateHandle::GenerateNewHandle), MoveTemp(Func)));
	}

	/**
	 * Register a custom event handler for this session
	 */
	template<typename EventType, typename HandlerType>
	FDelegateHandle RegisterCustomEventHandler(HandlerType* Handler, typename TConcertRawSessionCustomEventHandler<EventType, HandlerType>::FFuncType Func)
	{
		return InternalRegisterCustomEventHandler(EventType::StaticStruct()->GetFName(), MakeShared<TConcertRawSessionCustomEventHandler<EventType, HandlerType>>(FDelegateHandle(FDelegateHandle::GenerateNewHandle), Handler, Func));
	}

	/**
	 * Unregister a custom event handler for this session
	 */
	template<typename EventType>
	void UnregisterCustomEventHandler(const FDelegateHandle EventHandle)
	{
		InternalUnregisterCustomEventHandler(EventType::StaticStruct()->GetFName(), EventHandle);
	}

	/**
	 * Unregister a custom event handler for this session
	 */
	template<typename EventType, typename HandlerType>
	void UnregisterCustomEventHandler(HandlerType* EventHandler)
	{
		InternalUnregisterCustomEventHandler(EventType::StaticStruct()->GetFName(), EventHandler);
	}

	/**
	 * Clear a custom event handler for this session
	 */
	template<typename EventType>
	void ClearCustomEventHandler()
	{
		InternalClearCustomEventHandler(EventType::StaticStruct()->GetFName());
	}

	/**
	 * Send a custom event event to the given endpoint
	 */
	template<typename EventType>
	void SendCustomEvent(const EventType& Event, const FGuid& DestinationEndpointId, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> SequencedId = {})
	{
		InternalSendCustomEvent(EventType::StaticStruct(), &Event, TArrayBuilder<FGuid>().Add(DestinationEndpointId), Flags, SequencedId);
	}

	/**
	 * Send a custom event event to the given endpoints
	 */
	template<typename EventType>
	void SendCustomEvent(const EventType& Event, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> SequencedId = {})
	{
		InternalSendCustomEvent(EventType::StaticStruct(), &Event, DestinationEndpointIds, Flags, SequencedId);
	}

	/**
	 * Register a custom request handler for this session
	 */
	template<typename RequestType, typename ResponseType>
	void RegisterCustomRequestHandler(typename TConcertFunctionSessionCustomRequestHandler<RequestType, ResponseType>::FFuncType Func)
	{
		InternalRegisterCustomRequestHandler(RequestType::StaticStruct()->GetFName(), MakeShared<TConcertFunctionSessionCustomRequestHandler<RequestType, ResponseType>>(MoveTemp(Func)));
	}

	/**
	 * Register a custom request handler for this session
	 */
	template<typename RequestType, typename ResponseType, typename HandlerType>
	void RegisterCustomRequestHandler(HandlerType* Handler, typename TConcertRawSessionCustomRequestHandler<RequestType, ResponseType, HandlerType>::FFuncType Func)
	{
		InternalRegisterCustomRequestHandler(RequestType::StaticStruct()->GetFName(), MakeShared<TConcertRawSessionCustomRequestHandler<RequestType, ResponseType, HandlerType>>(Handler, Func));
	}

	/**
	 * Unregister a custom request handler for this session
	 */
	template<typename RequestType>
	void UnregisterCustomRequestHandler()
	{
		InternalUnregisterCustomRequestHandler(RequestType::StaticStruct()->GetFName());
	}

	/**
	 * Send a custom request to the given endpoint
	 */
	template<typename RequestType, typename ResponseType>
	TFuture<ResponseType> SendCustomRequest(const RequestType& Request, const FGuid& DestinationEndpointId)
	{
		TSharedRef<TConcertFutureSessionCustomResponseHandler<ResponseType>> Handler = MakeShared<TConcertFutureSessionCustomResponseHandler<ResponseType>>(); // TODO: this could be generalized to an ErasedPromise or something
		InternalSendCustomRequest(RequestType::StaticStruct(), &Request, DestinationEndpointId, Handler);
		return Handler->GetFuture();
	}

protected:
	/**
	 * Register a custom event handler for this session
	 */
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) = 0;

	/**
	 * Unregister a custom event handler for this session
	 */
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) = 0;
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) = 0;

	/**
	 * Clear a custom event handler for this session
	 */
	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) = 0;

	/**
	 * Send a custom event event to the given endpoints
	 */
	virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) = 0;

	/**
	 * Register a custom request handler for this session
	 */
	virtual void InternalRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler) = 0;

	/**
	 * Unregister a custom request handler for this session
	 */
	virtual void InternalUnregisterCustomRequestHandler(const FName& RequestMessageType) = 0;

	/**
	 * Send a custom request to the given endpoint
	 */
	virtual void InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler) = 0;

};

/** Interface for Concert server sessions */
class IConcertServerSession : public IConcertSession
{
public:
	virtual ~IConcertServerSession() = default;

	/** Rename the session. */
	virtual void SetName(const FString& NewName) = 0;

	/** Gets the client's address used by the messaging system. Can be fed into  */
	virtual FMessageAddress GetClientAddress(const FGuid& ClientEndpointId) const = 0;

	/** Callback when a server session gets ticked */
	virtual FOnConcertServerSessionTick& OnTick() = 0;

	/** Callback when a session client state changes */
	virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() = 0;

	/** Callback when a session message is acknowledged */
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() = 0;
};

/** Interface for Concert client sessions */
class IConcertClientSession : public IConcertSession
{
public:
	virtual ~IConcertClientSession() = default;

	/** Get the session connection status to the server session */
	virtual EConcertConnectionStatus GetConnectionStatus() const = 0;

	/** Get the client endpoint ID */
	virtual FGuid GetSessionClientEndpointId() const = 0;

	/** Get the server endpoint ID */
	virtual FGuid GetSessionServerEndpointId() const = 0;

	/** Get the local user's ClientInfo */
	virtual const FConcertClientInfo& GetLocalClientInfo() const = 0;

	/** Update the local user's ClientInfo */
	virtual void UpdateLocalClientInfo(const FConcertClientInfoUpdate& UpdatedFields) = 0;

	/** Start the connection handshake with the server session */
	virtual void Connect() = 0;

	/** Disconnect  gracefully from the server session */
	virtual void Disconnect() = 0;

	/** Manager for helping sequence custom events on client sessions. */
	virtual FConcertSequencedCustomEventManager& GetSequencedEventManager() = 0;

	/** Get the send/receive state for this session. */
	virtual EConcertSendReceiveState GetSendReceiveState() const = 0;

	/** Set the send/receive state for this session. */
	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) = 0;

	/** Callback when a connected client session gets ticked */
	virtual FOnConcertClientSessionTick& OnTick() = 0;

	/** Callback when the session connection state changes */
	virtual FOnConcertClientSessionConnectionChanged& OnConnectionChanged() = 0;

	/** Callback when a session client state changes */
	virtual FOnConcertClientSessionClientChanged& OnSessionClientChanged() = 0;

	/** Callback when the session name changes. */
	virtual FOnConcertSessionRenamed& OnSessionRenamed() = 0;
};
