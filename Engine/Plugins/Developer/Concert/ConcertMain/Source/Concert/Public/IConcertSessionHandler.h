// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Async/Future.h"
#include "Scratchpad/ConcertScratchpadPtr.h"

#include "ConcertMessages.h"

/** Context struct for session custom message handlers */
struct FConcertSessionContext
{
	FGuid SourceEndpointId;
	FGuid MessageId;
	EConcertMessageFlags MessageFlags;
	FConcertScratchpadPtr SenderScratchpad;
};

/** Interface for session custom event handler */
class IConcertSessionCustomEventHandler
{
public:
	virtual ~IConcertSessionCustomEventHandler() = default;

	virtual FDelegateHandle GetHandle() const = 0;

	virtual bool HasSameObject(const void* InObject) const = 0;

	virtual void HandleEvent(const FConcertSessionContext& Context, const void* EventData) const = 0;
};

/**
 * Common implementation of a session custom event handler.
 */
class FConcertSessionCustomEventHandlerBase : public IConcertSessionCustomEventHandler
{
public:
	explicit FConcertSessionCustomEventHandlerBase(FDelegateHandle InHandle)
		: Handle(MoveTemp(InHandle))
	{
	}

	virtual FDelegateHandle GetHandle() const override
	{
		return Handle;
	}

protected:
	FDelegateHandle Handle;
};

/**
 * Implementation of a session custom event handler that uses a raw member function pointer with the correct event type in the handler function signature.
 */
template<typename EventType, typename HandlerType>
class TConcertRawSessionCustomEventHandler : public FConcertSessionCustomEventHandlerBase
{
public:
	typedef void (HandlerType::*FFuncType)(const FConcertSessionContext&, const EventType&);

	TConcertRawSessionCustomEventHandler(FDelegateHandle InHandle, HandlerType* InHandler, FFuncType InFunc)
		: FConcertSessionCustomEventHandlerBase(MoveTemp(InHandle))
		, Handler(InHandler)
		, Func(InFunc)
	{
		check(Handler);
		check(Func);
	}

	virtual bool HasSameObject(const void* InObject) const override
	{
		return InObject == Handler;
	}

	virtual void HandleEvent(const FConcertSessionContext& Context, const void* EventData) const override
	{
		(Handler->*Func)(Context, *(const EventType*)EventData);
	}

private:
	HandlerType* Handler;
	FFuncType Func;
};

/**
 * Implementation of a session custom event handler that calls a function with the correct event type in the handler function signature.
 */
template<typename EventType>
class TConcertFunctionSessionCustomEventHandler : public FConcertSessionCustomEventHandlerBase
{
public:
	typedef TFunction<void(const FConcertSessionContext&, const EventType&)> FFuncType;

	TConcertFunctionSessionCustomEventHandler(FDelegateHandle InHandle, FFuncType InFunc)
		: FConcertSessionCustomEventHandlerBase(MoveTemp(InHandle))
		, Func(MoveTemp(InFunc))
	{
		check(Func);
	}
	
	virtual bool HasSameObject(const void* InObject) const override
	{
		return false;
	}

	virtual void HandleEvent(const FConcertSessionContext& Context, const void* EventData) const override
	{
		Func(Context, *(const EventType*)EventData);
	}

private:
	/** The Handler function */
	FFuncType Func;
};

/** Interface for session custom request handler */
class IConcertSessionCustomRequestHandler
{
public:
	virtual ~IConcertSessionCustomRequestHandler() = default;

	virtual UScriptStruct* GetResponseType() const = 0;

	virtual EConcertSessionResponseCode HandleRequest(const FConcertSessionContext& Context, const void* RequestData, void* ResponseData) const = 0;
};

/**
* Implementation of a session custom request handler that calls a raw member function with the correct request type in the handler function signature.
*/
template<typename RequestType, typename ResponseType, typename HandlerType>
class TConcertRawSessionCustomRequestHandler : public IConcertSessionCustomRequestHandler
{
public:
	typedef EConcertSessionResponseCode (HandlerType::*FFuncType)(const FConcertSessionContext&, const RequestType&, ResponseType&);

	TConcertRawSessionCustomRequestHandler(HandlerType* InHandler, FFuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(Handler);
		check(Func);
	}

	virtual ~TConcertRawSessionCustomRequestHandler() = default;

	virtual UScriptStruct* GetResponseType() const override
	{
		return ResponseType::StaticStruct();
	}

	virtual EConcertSessionResponseCode HandleRequest(const FConcertSessionContext& Context, const void* RequestData, void* ResponseData) const override
	{
		return (Handler->*Func)(Context, *(const RequestType*)RequestData, *(ResponseType*)ResponseData);
	}

private:
	HandlerType* Handler;
	FFuncType Func;
};

/**
 * Implementation of a session custom request handler that calls a function with the correct request type in the handler function signature.
 */
template<typename RequestType, typename ResponseType>
class TConcertFunctionSessionCustomRequestHandler : public IConcertSessionCustomRequestHandler
{
public:
	typedef TFunction<EConcertSessionResponseCode(const FConcertSessionContext&, const RequestType&, ResponseType&)> FFuncType;

	TConcertFunctionSessionCustomRequestHandler(FFuncType InFunc)
		: Func(MoveTemp(InFunc))
	{
		check(Func);
	}

	virtual ~TConcertFunctionSessionCustomRequestHandler() = default;

	virtual UScriptStruct* GetResponseType() const override
	{
		return ResponseType::StaticStruct();
	}

	virtual EConcertSessionResponseCode HandleRequest(const FConcertSessionContext& Context, const void* RequestData, void* ResponseData) const override
	{
		return Func(Context, *(const RequestType*)RequestData, *(ResponseType*)ResponseData);
	}

private:
	FFuncType Func;
};

/** Interface for session custom request handler  TODO: replace by a generalized erased promise/future pair?? */
class IConcertSessionCustomResponseHandler
{
public:
	virtual ~IConcertSessionCustomResponseHandler() = default;

	virtual void HandleResponse(const void* ResponseData) = 0;
};

/**
 * Implementation of a session custom response handler that uses a future to dispatch back the response
 */
template<typename ResponseType>
class TConcertFutureSessionCustomResponseHandler : public IConcertSessionCustomResponseHandler
{
public:
	virtual ~TConcertFutureSessionCustomResponseHandler() = default;

	virtual void HandleResponse(const void* ResponseData) override
	{
		// TODO: add checks?
		if (ResponseData != nullptr)
		{
			Promise.SetValue(*(const ResponseType*)ResponseData);
		}
		else
		{
			// Default constructed response should be considered an error or properly handled
			Promise.SetValue(ResponseType());
		}
	}

	TFuture<ResponseType> GetFuture()
	{
		return Promise.GetFuture();
	}
private:
	TPromise<ResponseType> Promise;
};
