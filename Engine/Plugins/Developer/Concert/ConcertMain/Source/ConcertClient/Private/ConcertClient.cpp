// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClient.h"

#include "ConcertClientSession.h"
#include "ConcertMessages.h"
#include "ConcertUtil.h"
#include "ConcertLogGlobal.h"
#include "ConcertTransportEvents.h"
#include "ConcertClientSettings.h"

#include "Algo/Transform.h"

#include "Containers/Ticker.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"
#include "Misc/AsyncTaskNotification.h"
#include "HAL/FileManager.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Stats/Stats.h"

#include "Framework/Commands/UIAction.h"

#include "Runtime/Launch/Resources/Version.h"

#define LOCTEXT_NAMESPACE "ConcertClient"

LLM_DEFINE_TAG(Concert_ConcertClient);

namespace ConcertUtil
{
	const FLogCategoryBase* GetLogConcertPtr()
	{
	#if NO_LOGGING
		return nullptr;
	#else
		return &LogConcert;
	#endif
	}

	// Connection Error code
	constexpr uint32 CancelCode = 1;
	constexpr uint32 ConectionAttemptAbortedErrorCode = 2;
	constexpr uint32 ServerNotRespondingErrorCode = 3;
	constexpr uint32 ServerErrorCode = 4;
}

class FConcertAutoConnection
{
public:
	FConcertAutoConnection(FConcertClient* InClient, const UConcertClientConfig* InSettings)
		: Client(InClient)
		, Settings(InSettings)
	{
		// Make sure discovery is enabled on the client
		Client->StartDiscovery();
		Client->OnSessionConnectionChanged().AddRaw(this, &FConcertAutoConnection::HandleConnectionChanged);
		Client->OnSessionStartup().AddRaw(this, &FConcertAutoConnection::HandleSessionStartup);

		AutoConnectionNotification = MakeAutoConnectNotification();

		AutoConnectionTickHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ConcertAutoConnect"), 1, [this](float)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FConcertAutoConnection_Tick);
			Tick();
			return true;
		});
	}

	~FConcertAutoConnection()
	{
		Client->StopDiscovery();
		Client->OnSessionConnectionChanged().RemoveAll(this);
		Client->OnSessionStartup().RemoveAll(this);

		if (AutoConnectionTickHandle.IsValid())
		{
			FTSTicker::RemoveTicker(AutoConnectionTickHandle);
			AutoConnectionTickHandle.Reset();
		}

		if (AutoConnectionNotification) // Abort if it still ongoing.
		{
			AutoConnectionNotification->SetKeepOpenOnFailure(false); // Don't keep it open on abort.
			AutoConnectionNotification->SetComplete(GetAutoConnectionCanceledMessage(), FText::GetEmpty(), false);
		}
	}

private:
	FText GetAutoConnectionCanceledMessage() const
	{
		return FText::Format(LOCTEXT("AutoJoinSessionCanceled", "Connection to Session '{0}' Canceled."), FText::AsCultureInvariant(Settings->DefaultSessionName));
	}

	TUniquePtr<FAsyncTaskNotification> MakeAutoConnectNotification()
	{
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.TitleText = FText::Format(LOCTEXT("AutoJoinSession", "Joining Session '{0}' on '{1}'..."), FText::AsCultureInvariant(Settings->DefaultSessionName), FText::AsCultureInvariant((Settings->DefaultServerURL)));
		NotificationConfig.ProgressText = FText::Format(LOCTEXT("LookingForServer", "Looking for Server '{0}'..."), FText::AsCultureInvariant((Settings->DefaultServerURL)));
		NotificationConfig.bIsHeadless = Settings->bIsHeadless;
		NotificationConfig.bCanCancel = true;
		NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();
		return MakeUnique<FAsyncTaskNotification>(NotificationConfig);
	}

	void SetAsyncNotificationComplete(const FText& Msg, bool bSucceeded)
	{
		if (AutoConnectionNotification.IsValid())
		{
			AutoConnectionNotification->SetComplete(Msg, FText::GetEmpty(), bSucceeded);
			AutoConnectionNotification.Reset();
		}
	}

	void Tick()
	{
		// Already connected
		if (IsConnected())
		{
			// Once connected if we aren't in auto connection mode, shut ourselves down
			if (!Settings->bAutoConnect)
			{
				Client->StopAutoConnect(); // Indirect self-destruct.
			}
			return;
		}

		// Should cancel request?
		if (AutoConnectionNotification.IsValid() && AutoConnectionNotification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel)
		{
			SetAsyncNotificationComplete(GetAutoConnectionCanceledMessage(), false);
			Client->StopAutoConnect(); // Indirect self-destruct.
			return;
		}

		// A create or join request is ongoing.
		if (OngoingConnectionRequest.IsValid())
		{
			if (OngoingConnectionRequest.IsReady())
			{
				TSharedFuture<EConcertResponseCode> SessionJoined = OngoingConnectionRequest.Get();
				if (SessionJoined.IsReady())
				{
					const EConcertResponseCode RequestResponseCode = SessionJoined.Get();
					if (RequestResponseCode != EConcertResponseCode::Success)
					{
						// If last attempt failed, stop trying if auto-connect was turned off or if we are not allowed to retry on error.
						if (RequestResponseCode == EConcertResponseCode::Failed && (!Settings->bAutoConnect || !Settings->bRetryAutoConnectOnError))
						{
							check(!AutoConnectionNotification.IsValid()); // If OngoingConnectionRequest is valid, the notification ownership was transfered to it.
							Client->StopAutoConnect(); // Indirect self-destruct.
							return;
						}
					}

					// This ongoing request has completed, reset to retry later.
					OngoingConnectionRequest = TFuture<TSharedFuture<EConcertResponseCode>>();
				}
			}
			return;
		}

		// Ensure to display an async notification.
		if (!AutoConnectionNotification.IsValid()) // Invalid if ownership was transfered to a failed create/join or if the session was joined and server went down.
		{
			AutoConnectionNotification = MakeAutoConnectNotification();
		}

		check(!IsConnecting()); // If it fails -> most likely because a 'cancel' occurred while connecting, but the cancel did not 'disconnect' the local session before the ongoing connection promise value was set.

		// Clear our current session before initiating a new connection request
		CurrentSession.Reset();

		// Try to create or/and join the session.
		for (const FConcertServerInfo& ServerInfo : Client->GetKnownServers())
		{
			if (ServerInfo.ServerName == Settings->DefaultServerURL)
			{
				CreateOrJoinDefaultSession(ServerInfo);
				break; // Can only have one default server.
			}
		}
	}

	bool IsConnected() const
	{
		return CurrentSession.IsValid() ? CurrentSession.Pin()->GetConnectionStatus() == EConcertConnectionStatus::Connected : false;
	}

	bool IsConnecting() const
	{
		return CurrentSession.IsValid() ? CurrentSession.Pin()->GetConnectionStatus() == EConcertConnectionStatus::Connecting : false;
	}

	void CreateOrJoinDefaultSession(const FConcertServerInfo& ServerInfo)
	{
		// Prevents TFuture execution if this class gets deleted before TFuture executes (and dismisses previous execution if any).
		AsyncRequestExecutionGuard = MakeShared<uint8>();
		TWeakPtr<uint8> AsyncRequestExecutionToken = AsyncRequestExecutionGuard;

		// Get the Server sessions list
		OngoingConnectionRequest = Client->GetServerSessions(ServerInfo.AdminEndpointId)
			.Next([LocalSettings = Settings](FConcertAdmin_GetAllSessionsResponse Response)
			{
				FGuid DefaultSessionId;
				FGuid DefaultSessionToRestoreId;
				if (Response.ResponseCode == EConcertResponseCode::Success)
				{
					// Find our default session IDs
					for (const FConcertSessionInfo& SessionInfo : Response.LiveSessions)
					{
						if (SessionInfo.SessionName == LocalSettings->DefaultSessionName)
						{
							DefaultSessionId = SessionInfo.SessionId;
							break;
						}
					}
					for (const FConcertSessionInfo& SessionInfo : Response.ArchivedSessions)
					{
						if (SessionInfo.SessionName == LocalSettings->DefaultSessionToRestore)
						{
							DefaultSessionToRestoreId = SessionInfo.SessionId;
							break;
						}
					}
				}
				return MakeTuple(Response.ResponseCode, DefaultSessionId, DefaultSessionToRestoreId);
			})
			.Next([this, AsyncRequestExecutionToken, ServerEndpoint = ServerInfo.AdminEndpointId](TTuple<EConcertResponseCode, FGuid, FGuid> RequestLiveSessionTuple)
			{
				const EConcertResponseCode OuterResponseCode = RequestLiveSessionTuple.Get<0>();

				// The request was successful and execution not dismissed. (and 'this' is also valid)
				if (OuterResponseCode == EConcertResponseCode::Success && AsyncRequestExecutionToken.IsValid())
				{
					const FGuid DefaultSessionId = RequestLiveSessionTuple.Get<1>();
					const FGuid DefaultSessionToRestoreId = RequestLiveSessionTuple.Get<2>();

					// We found the default session, join it.
					if (DefaultSessionId.IsValid())
					{
						return Client->InternalJoinSession(ServerEndpoint, DefaultSessionId, MoveTemp(AutoConnectionNotification)).Share();
					}
					
					// We found the default session to restore, restore and join it.
					if (DefaultSessionToRestoreId.IsValid())
					{
						FConcertCopySessionArgs RestoreSessionArgs;
						RestoreSessionArgs.bAutoConnect = true;
						RestoreSessionArgs.SessionId = DefaultSessionToRestoreId;
						RestoreSessionArgs.SessionName = Settings->DefaultSessionName;
						RestoreSessionArgs.ArchiveNameOverride = Settings->DefaultSaveSessionAs;
						return Client->InternalCopySession(ServerEndpoint, RestoreSessionArgs, /*bRestoreOnlyConstraint*/true, MoveTemp(AutoConnectionNotification)).Share();
					}

					// No session found to join or restore, so create a new one.
					{
						FConcertCreateSessionArgs CreateSessionArgs;
						CreateSessionArgs.SessionName = Settings->DefaultSessionName;
						CreateSessionArgs.ArchiveNameOverride = Settings->DefaultSaveSessionAs;
						return Client->InternalCreateSession(ServerEndpoint, CreateSessionArgs, MoveTemp(AutoConnectionNotification)).Share();
					}
				}
				else // Request failed, was canceled or the captured 'this' was deleted.
				{
					return MakeFulfilledPromise<EConcertResponseCode>(OuterResponseCode == EConcertResponseCode::Success ? EConcertResponseCode::Failed : OuterResponseCode).GetFuture().Share();
				}
			});
	}

	void HandleConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
	{
		// Once we get connected or disconnected, clear our ongoing request if we have one, if it comes from our current session
		if (CurrentSession.IsValid() 
			&& CurrentSession.Pin().Get() == &InSession
			&& (ConnectionStatus == EConcertConnectionStatus::Connected || ConnectionStatus == EConcertConnectionStatus::Disconnected))
		{
			OngoingConnectionRequest = TFuture<TSharedFuture<EConcertResponseCode>>();
		}
	}

	void HandleSessionStartup(TSharedRef<IConcertClientSession> InSession)
	{
		CurrentSession = InSession;
	}

	TFuture<TSharedFuture<EConcertResponseCode>> OngoingConnectionRequest;
	FTSTicker::FDelegateHandle AutoConnectionTickHandle;
	FConcertClient* Client;
	TWeakPtr<IConcertClientSession> CurrentSession;
	const UConcertClientConfig* Settings;
	TUniquePtr<FAsyncTaskNotification> AutoConnectionNotification;
	TSharedPtr<uint8> AsyncRequestExecutionGuard;
};

/** Runs a set of tasks required to join a session. */
class FConcertPendingConnection : public TSharedFromThis<FConcertPendingConnection>
{
public:
	struct FConfig
	{
		TAttribute<FText> PendingTitleText;
		TAttribute<FText> SuccessTitleText;
		TAttribute<FText> FailureTitleText;
		TAttribute<bool> KeepNotificationOpenOnError;
		bool bIsAutoConnection = false;
	};

	FConcertPendingConnection(FConcertClient* InClient, const FConfig& InConfig)
		: Client(InClient)
		, Config(InConfig)
	{
	}

	~FConcertPendingConnection()
	{
		if (ConnectionTick.IsValid())
		{
			FTSTicker::RemoveTicker(ConnectionTick);
		}

		if (ConnectionTasks.Num() > 0)
		{
			// Don't keep the notification open if canceled/aborted.
			Notification->SetKeepOpenOnFailure(false);

			// Abort the executing tasks.
			ConnectionTasks[0]->Abort();

			// Clear the tasks, set the notification text and fulfill the 'Execute()' promise.
			SetResult(EConcertResponseCode::Failed, GetCanceledError());
		}
	}

	/** Execute the connection. When the future is ready, the client is whether connected or not. The task may be canceled at anytime. */
	TFuture<EConcertResponseCode> Execute(TArray<TUniquePtr<IConcertClientConnectionTask>>&& InConnectionTasks, TUniquePtr<FAsyncTaskNotification> OngoingNotification)
	{
		checkf(ConnectionTasks.Num() == 0, TEXT("Execute has already been called!"));
		ConnectionTasks = MoveTemp(InConnectionTasks);
		checkf(ConnectionTasks.Num() != 0, TEXT("Execute was not given any tasks!"));

		// Set-up the task notification, continuing the ongoing one if any (like the auto connection one)
		Notification = MoveTemp(OngoingNotification);
		if (!Notification.IsValid())
		{
			FAsyncTaskNotificationConfig NotificationConfig;
			NotificationConfig.TitleText = Config.PendingTitleText.Get(FText::GetEmpty());
			NotificationConfig.bIsHeadless = Client->GetConfiguration()->bIsHeadless;
			NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();
			Notification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
		}
		Notification->SetCanCancel(TAttribute<bool>(this, &FConcertPendingConnection::CanCancel));
		Notification->SetKeepOpenOnFailure(!Config.bIsAutoConnection);
		Notification->SetProgressText(ConnectionTasks[0]->GetDescription());

		ConnectionTasks[0]->Execute();

		ConnectionTick = FTSTicker::GetCoreTicker().AddTicker(TEXT("ConcertPendingConnection"), 0.1f, [this](float)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FConcertPendingConnection_Tick);
			Tick();
			return true;
		});

		return ConnectionResult.GetFuture();
	}

private:
	static FConcertConnectionError GetCanceledError()
	{
		return FConcertConnectionError{ ConcertUtil::CancelCode, LOCTEXT("ConnectionProcessCanceled", "Connection Process Canceled") };
	}

	bool CanCancel() const
	{
		return ConnectionTasks.Num() > 0 && ConnectionTasks[0]->CanCancel();
	}

	void Tick()
	{
		auto GetTaskAction = [](EAsyncTaskNotificationPromptAction InPromptAction) ->EConcertConnectionTaskAction
		{
			switch (InPromptAction)
			{
			case EAsyncTaskNotificationPromptAction::None:
				return EConcertConnectionTaskAction::None;
			case EAsyncTaskNotificationPromptAction::Cancel:
				return EConcertConnectionTaskAction::Cancel;
			case EAsyncTaskNotificationPromptAction::Continue:
				return EConcertConnectionTaskAction::Continue;
			// unattended case resolve as a continue
			default:
				return EConcertConnectionTaskAction::Continue;
			}
		};

		// We should only Tick while we have tasks to process
		check(ConnectionTasks.Num() > 0);

		EAsyncTaskNotificationPromptAction PromptAction = Notification->GetPromptAction();
		EConcertConnectionTaskAction TaskAction = GetTaskAction(PromptAction);
		const bool bCanceled = TaskAction == EConcertConnectionTaskAction::Cancel;
		EConcertResponseCode TaskStatus = ConnectionTasks[0]->GetStatus();
		if (bCanceled)
		{
			if (TaskStatus == EConcertResponseCode::Pending)
			{
				ConnectionTasks[0]->Tick(TaskAction); // Give it a last tick to give it a chance to cancel cleanly.
			}

			SetResultAndDelete(EConcertResponseCode::Failed, bCanceled, GetCanceledError()); // Cancelation has priority over other possible errors (it could hide some failure)
			return; // Do not use 'this' anymore, it was deleted above.
		}

		// Update the current task
		switch (TaskStatus)
		{
			// Pending state - update the task
		case EConcertResponseCode::Pending:
			ConnectionTasks[0]->Tick(TaskAction);
			return;
			// Prompt state - wait for user action to either proceed (success) or stop (fail)
		case EConcertResponseCode::InvalidRequest:
			{
				FAsyncNotificationStateData StateData(Config.PendingTitleText.Get(FText::GetEmpty()), ConnectionTasks[0]->GetError().ErrorText, EAsyncTaskNotificationState::Prompt);
				StateData.PromptText = ConnectionTasks[0]->GetPrompt();
				StateData.HyperlinkText = LOCTEXT("PendingConnectionFailureDetails", "See Details...");
				StateData.Hyperlink = ConnectionTasks[0]->GetErrorDelegate();
				Notification->SetNotificationState(StateData);
				ConnectionTasks[0]->Tick(TaskAction);
				return;
			}
			// Success state - move on to the next task
		case EConcertResponseCode::Success:
			ConnectionTasks.RemoveAt(0, EAllowShrinking::No);
			if (ConnectionTasks.Num() > 0)
			{
				Notification->SetNotificationState(FAsyncNotificationStateData(Config.PendingTitleText.Get(FText::GetEmpty()), ConnectionTasks[0]->GetDescription(), EAsyncTaskNotificationState::Pending));
				ConnectionTasks[0]->Execute();
			}
			else
			{
				// Processed everything without error
				SetResultAndDelete(EConcertResponseCode::Success, bCanceled); // do not use 'this' after this call!
			}
			return;
			// Error state - fail the connection
		default:
			SetResultAndDelete(TaskStatus, bCanceled, ConnectionTasks[0]->GetError(), ConnectionTasks[0]->GetErrorDelegate()); // do not use 'this' after this call!
			return;
		}
	}

	/** Set the result */
	void SetResult(const EConcertResponseCode InResult, const FConcertConnectionError InError = FConcertConnectionError(), const FSimpleDelegate& InErrorDelegate = FSimpleDelegate())
	{
		if (InResult == EConcertResponseCode::Success)
		{
			Notification->SetComplete(Config.SuccessTitleText.Get(FText::GetEmpty()), FText(), /*bSuccess*/true);
		}
		else
		{
			Client->InternalDisconnectSession();
			if (InResult == EConcertResponseCode::Failed)
			{
				Notification->SetKeepOpenOnFailure(Config.KeepNotificationOpenOnError);
			}
			Notification->SetHyperlink(InErrorDelegate, LOCTEXT("PendingConnectionFailureDetails", "See Details..."));
			Notification->SetComplete(Config.FailureTitleText.Get(FText::GetEmpty()), InError.ErrorText, /*bSuccess*/false);
		}

		ConnectionTasks.Reset();
		ConnectionResult.SetValue(InResult);
		Client->SetLastConnectionError(InError);
	}

	/** Set the result and delete ourself - 'this' will be garbage after calling this function! */
	void SetResultAndDelete(const EConcertResponseCode InResult, bool bWasCanceled, const FConcertConnectionError InError = FConcertConnectionError(), const FSimpleDelegate& InErrorDelegate = FSimpleDelegate())
	{
		// Set the result and delete ourself
		SetResult(InResult, InError, InErrorDelegate);
		check(Client->PendingConnection.Get() == this);
		// if the connection was canceled, also cancel the auto connection, so it won't retry on failure		
		if (bWasCanceled)
		{
			Client->AutoConnection.Reset();
		}
		Client->PendingConnection.Reset();
	}

	FConcertClient* Client;
	FConfig Config;
	FTSTicker::FDelegateHandle ConnectionTick;
	TPromise<EConcertResponseCode> ConnectionResult;
	TUniquePtr<FAsyncTaskNotification> Notification;
	TArray<TUniquePtr<IConcertClientConnectionTask>> ConnectionTasks;
};

template <typename RequestType>
class TConcertClientConnectionRequestTask : public IConcertClientConnectionTask
{
public:
	TConcertClientConnectionRequestTask(FConcertClient* InClient, RequestType&& InRequest, const FGuid& InServerAdminEndpointId)
		: Client(InClient)
		, Request(MoveTemp(InRequest))
		, ServerAdminEndpointId(InServerAdminEndpointId)
		, AsyncRequestExecutionGuard(MakeShared<uint8>(0))
	{
	}

	virtual void Abort() override
	{
		Result.Reset();
	}

	virtual void Tick(EConcertConnectionTaskAction TaskAction) override
	{
	}

	virtual bool CanCancel() const override
	{
		return true;
	}

	virtual EConcertResponseCode GetStatus() const override
	{
		if (Result.IsValid())
		{
			if (!Result.IsReady())
			{
				return EConcertResponseCode::Pending;
			}

			TSharedFuture<EConcertResponseCode> SessionJoined = Result.Get();
			if (SessionJoined.IsValid())
			{
				return SessionJoined.IsReady() ? SessionJoined.Get() : EConcertResponseCode::Pending;
			}
		}

		return EConcertResponseCode::Failed;
	}

	virtual FText GetPrompt() const override
	{
		// client connection task have no prompt
		return FText::GetEmpty();
	}

	virtual FConcertConnectionError GetError() const override
	{
		return Result.IsValid() ? ConnectionError : FConcertConnectionError{ConcertUtil::ConectionAttemptAbortedErrorCode ,LOCTEXT("RemoteConnectionAttemptAborted", "Remote Connection Attempt Aborted.") };
	}

	virtual FSimpleDelegate GetErrorDelegate() const override
	{
		return FSimpleDelegate();
	}

	virtual FText GetDescription() const override
	{
		return LOCTEXT("AttemptingRemoteConnection", "Attempting Remote Connection...");
	}

	static FConcertConnectionError GetServerNotRespondingErrorMessage()
	{
		return FConcertConnectionError{ ConcertUtil::ServerNotRespondingErrorCode, LOCTEXT("JoinTask_ServerNotResponding", "Server Not Responding") };
	}

protected:
	FConcertClient* Client;
	RequestType Request;
	FGuid ServerAdminEndpointId;
	TFuture<TSharedFuture<EConcertResponseCode>> Result;
	FConcertConnectionError ConnectionError;
	TSharedPtr<uint8> AsyncRequestExecutionGuard;
};

class FConcertClientJoinSessionTask : public TConcertClientConnectionRequestTask<FConcertAdmin_FindSessionRequest>
{
public:
	FConcertClientJoinSessionTask(FConcertClient* InClient, FConcertAdmin_FindSessionRequest&& InRequest, const FGuid& InServerAdminEndpointId, TSharedRef<FText> OutResolvedSessionName)
		: TConcertClientConnectionRequestTask(InClient, MoveTemp(InRequest), InServerAdminEndpointId)
		, ResolvedSessionName(MoveTemp(OutResolvedSessionName))
	{
	}

	virtual void Execute() override
	{
		TWeakPtr<uint8> AsyncExecutionToken = AsyncRequestExecutionGuard;
		Result = Client->ClientAdminEndpoint->SendRequest<FConcertAdmin_FindSessionRequest, FConcertAdmin_SessionInfoResponse>(Request, ServerAdminEndpointId)
			.Next([this, AsyncExecutionToken](const FConcertAdmin_SessionInfoResponse& SessionInfoResponse)
			{
				if (!AsyncExecutionToken.IsValid()) // The task was canceled/aborted and the object deleted. Don't care about the return value/error.
				{
					return MakeFulfilledPromise<EConcertResponseCode>(EConcertResponseCode::Failed).GetFuture().Share();
				}

				*ResolvedSessionName = FText::AsCultureInvariant(SessionInfoResponse.SessionInfo.SessionName);
				if (SessionInfoResponse.ResponseCode == EConcertResponseCode::Success)
				{
					if (SessionInfoResponse.FileShare.bFileShareEnabled && Client->FileSharingService)
					{
						// Enable the file sharing service.
						Client->FileSharingService->SetEnabled(true);

						// Clean-up the challenge file we originally generated when we joined the session.
						Client->FileSharingService->Delete(SessionInfoResponse.FileShare.Challenge.Path);
					}

					// If CreateClientSession() returns a failure, it is because the server did not reply to the 'join session' event and the endpoint timed out or the connection was canceled/aborted (for which there is a special message already).
					ConnectionError = GetServerNotRespondingErrorMessage();
					return Client->CreateClientSession(SessionInfoResponse.SessionInfo).Share();
				}
				else
				{
					ConnectionError.ErrorCode = ConcertUtil::ServerErrorCode;
					ConnectionError.ErrorText = SessionInfoResponse.Reason;
					return MakeFulfilledPromise<EConcertResponseCode>(SessionInfoResponse.ResponseCode).GetFuture().Share();
				}
			});
	}

	TSharedRef<FText> ResolvedSessionName;
};

class FConcertClientCreateSessionTask : public TConcertClientConnectionRequestTask<FConcertAdmin_CreateSessionRequest>
{
public:
	FConcertClientCreateSessionTask(FConcertClient* InClient, FConcertAdmin_CreateSessionRequest&& InRequest, const FGuid& InServerAdminEndpointId)
		: TConcertClientConnectionRequestTask(InClient, MoveTemp(InRequest), InServerAdminEndpointId)
	{
	}

	virtual void Execute() override
	{
		TWeakPtr<uint8> AsyncExecutionToken = AsyncRequestExecutionGuard;
		Result = Client->ClientAdminEndpoint->SendRequest<FConcertAdmin_CreateSessionRequest, FConcertAdmin_SessionInfoResponse>(Request, ServerAdminEndpointId)
			.Next([this, AsyncExecutionToken](const FConcertAdmin_SessionInfoResponse& SessionInfoResponse)
			{
				if (!AsyncExecutionToken.IsValid()) // The task was canceled/aborted and the object deleted. Don't care about the return value/error.
				{
					return MakeFulfilledPromise<EConcertResponseCode>(EConcertResponseCode::Failed).GetFuture().Share();
				}
				else if (SessionInfoResponse.ResponseCode == EConcertResponseCode::Success)
				{
					if (SessionInfoResponse.FileShare.bFileShareEnabled && Client->FileSharingService)
					{
						// Enable the file sharing service.
						Client->FileSharingService->SetEnabled(true);

						// Clean-up the challenge file we originally generated when we joined the session.
						Client->FileSharingService->Delete(SessionInfoResponse.FileShare.Challenge.Path);
					}

					// If CreateClientSession() returns a failure, it is because the server did not reply to the 'join session' event and the endpoint timed out or the connection was canceled/aborted (for which there is a special message already).
					ConnectionError = GetServerNotRespondingErrorMessage();
					return Client->CreateClientSession(SessionInfoResponse.SessionInfo).Share();
				}
				else
				{
					ConnectionError.ErrorCode = ConcertUtil::ServerErrorCode;
					ConnectionError.ErrorText = SessionInfoResponse.Reason;
					return MakeFulfilledPromise<EConcertResponseCode>(SessionInfoResponse.ResponseCode).GetFuture().Share();
				}
			});
	}
};

FConcertClientPaths::FConcertClientPaths(const FString& InRole)
	: WorkingDir(FPaths::ProjectIntermediateDir() / TEXT("Concert") / InRole / FApp::GetInstanceId().ToString())
{
}


namespace UE::ConcertClient::Private
{

TAutoConsoleVariable<int32> CVarLongHitchDuration(
	TEXT("ConcertClient.Debug.GameThreadHitchDuration"),
	30,
	TEXT("How long to wait in seconds when ConcertClient.Debug.GameThreadHitch is called")
);

void ForceHitch()
{
	FPlatformProcess::Sleep(CVarLongHitchDuration.GetValueOnAnyThread());
}
	
FAutoConsoleCommand StartTracingCommand(TEXT("ConcertClient.Debug.GameThreadHitch"),
	TEXT("Force a long hitch on game thread."),
	FExecuteAction::CreateStatic(&ForceHitch));

FConcertSessionFileShare MakeFileSharingChallenge(TSharedPtr<IConcertFileSharingService>& SharingService, const FDirectoryPath& FileShareRoot)
{
	FConcertSessionFileShare FileShare;
	FileShare.bFileShareEnabled = false;
	
	// We should always have a file sharing service to work with.
	if (SharingService)
	{
		FileShare.FileShareRoot = FileShareRoot.Path;
		
		SharingService->SetPath(FileShareRoot.Path);
		
		TSharedPtr<FArchive> Writer = SharingService->CreateWriter(FileShare.Challenge.Path);
		if (Writer)
		{
			FileShare.bFileShareEnabled = true;
			*Writer << FileShare.Challenge.Data.ChallengeKey;
			*Writer << FileShare.Challenge.Data.CreationTimeInTicks;
		}
	}

	if (!FileShare.bFileShareEnabled)
	{
		UE_LOGF(LogConcert, Display, "File sharing is disabled this client.");
	}

	return FileShare;
}
	
}; // namespace UE::ConcertClient::Private

class FConcertClientServerDiscoveryThread : public FRunnable
{
public:
	explicit FConcertClientServerDiscoveryThread(TSharedPtr<IConcertLocalEndpoint> InClientAdminEndpoint, FString InRole, FString InAuthKey, int32 TimeoutInSeconds)
		: bIsRunning(false)
		, bStopRequested(false)
		, ClientAdminEndpoint(MoveTemp(InClientAdminEndpoint))
	{
		Thread.Reset(FRunnableThread::Create(this, TEXT("ConcertClient Server Discovery Thread")));

		DiscoveryEventData.CurrentRole = MoveTemp(InRole);
		DiscoveryEventData.CurrentAuthKey = MoveTemp(InAuthKey);
		DiscoveryEventData.DiscoveryTimeoutSeconds = TimeoutInSeconds;
		ClientAdminEndpoint->RegisterEventHandler<FConcertAdmin_ServerDiscoveredEvent>(this, &FConcertClientServerDiscoveryThread::HandleServerDiscoveryEvent);
	}

	~FConcertClientServerDiscoveryThread()
	{
		if (Thread)
		{
			Thread->Kill(true);
			Thread = nullptr;
		}

	}

	void UnregisterHandler()
	{
		if (ClientAdminEndpoint.IsValid())
		{
			ClientAdminEndpoint->UnregisterEventHandler<FConcertAdmin_ServerDiscoveredEvent>();
		}

		ClientAdminEndpoint.Reset();
	}
		
	bool IsRunning() const
	{
		return Thread && bIsRunning;
	}

	/**
	 * Update the discovery information we send to the server. The user could change the timeout or
	 * current authentication keys from the client settings. 
	 **/
	void SetDiscoveryEventInformation(FString InRole, FString InAuthKey, int32 TimeoutInSeconds)
	{
		FScopeLock DiscoveryDataLock(&DiscoveryEventDataCS);
		DiscoveryEventData.CurrentRole = MoveTemp(InRole);
		DiscoveryEventData.CurrentAuthKey = MoveTemp(InAuthKey);
		DiscoveryEventData.DiscoveryTimeoutSeconds = TimeoutInSeconds;
	}
		
	/**
	 * For use by external callers to check to see if the server list has updated.  This is to allow the ticker
	 * on the game thread to check for updated server list and broadcast out to existing listeners.  The original
	 * implementation always existed on the game thread. This helps maintain that functionality and limit the
	 * need for delegates to be thread safe.
	 */
	bool HasServerListChanged() const
	{
		return bServersUpdated;
	}

	/**
	 * Reset the flag that tracks changes to the running server list.
	 */
	void ResetServerListChange()
	{
		bServersUpdated = false;
	}
	/**
	 * Gets the current list of known servers.  We have to lock access to the known server list before returning values. 
	 */
	TArray<FConcertServerInfo> GetKnownServers()
	{
		FScopeLock KnownServersLock(&KnownServersCS);
		TArray<FConcertServerInfo> ServerArray;
		ServerArray.Empty(KnownServers.Num());
		for (const auto& Server : KnownServers)
		{
			ServerArray.Emplace(Server.Value.ServerInfo);
		}
		return ServerArray;
	}

	//~ FRunnable Interface
	virtual bool Init() override
	{
		bIsRunning = true;
		return true;
	}

	/**
	 * Every second send out a discovery message to clients and record discovered servers. Timeout any servers that are no
	 * longer reporting online.
	 * */
	virtual uint32 Run() override
	{
		while (!bStopRequested)
		{
			const FDateTime UtcNow = FDateTime::UtcNow();
			SendDiscoverServersEvent();
			TimeoutDiscovery(UtcNow);
			FPlatformProcess::SleepNoStats(1.0f);
		}
		return 0;
	}

	/** Stop the running thread and unregister our discovery handlers. */
	virtual void Stop() override
	{
		UnregisterHandler();
		bStopRequested = true;
	}

	/** Thread was told to exit by the FRunnable interface. */
	virtual void Exit() override
	{
		bIsRunning = false;
	}
	//~ FRunnable Interface.

private:
	/**
	 * Dirty the server updated flag that indicates that a new server has been discovered.
	 */
	void SetServersUpdatedFlag()
	{
		bServersUpdated = true;
	}

	/**
	 * Handle a server discovery event and update our list of known Multi-user server.s
	 */
	void HandleServerDiscoveryEvent(const FConcertMessageContext& Context)
	{
		const FConcertAdmin_ServerDiscoveredEvent* Message = Context.GetMessage<FConcertAdmin_ServerDiscoveredEvent>();

		if (Message->ConcertProtocolVersion != EConcertMessageVersion::LatestVersion)
		{
			return;
		}

		FScopeLock KnownServersLock(&KnownServersCS);
		FKnownServer* Info = KnownServers.Find(Message->ConcertEndpointId);
		if (Info == nullptr)
		{
			UE_LOGF(LogConcert, Display, "Server %ls discovered.", *Message->ServerName);
			KnownServers.Emplace(Message->ConcertEndpointId, FKnownServer{ Context.UtcNow, FConcertServerInfo{ Message->ConcertEndpointId, Message->ServerName, Message->InstanceInfo, Message->ServerFlags } });
			
			SetServersUpdatedFlag();
		}
		else
		{
			Info->LastDiscoveryTime = Context.UtcNow;
		}
	}

	/**
	 * Send a discovery message to servers on the network. A server will send a response back which will be handled by
	 * HandleServerDiscoveryEvent above.
	 */
	void SendDiscoverServersEvent()
	{
		FScopeLock DiscoveryDataLock(&DiscoveryEventDataCS);
		FConcertAdmin_DiscoverServersEvent DiscoverServersEvent;
		DiscoverServersEvent.ConcertProtocolVersion = EConcertMessageVersion::LatestVersion;
		DiscoverServersEvent.RequiredRole = DiscoveryEventData.CurrentRole;;
		DiscoverServersEvent.RequiredVersion = VERSION_STRINGIFY(ENGINE_MAJOR_VERSION) TEXT(".") VERSION_STRINGIFY(ENGINE_MINOR_VERSION);
		DiscoverServersEvent.ClientAuthenticationKey = DiscoveryEventData.CurrentAuthKey;

		/** This call ends up into Message Bus which is thread safe.  This will be called even if the main thread is blocked so that we do not lose contact with running servers. */
		ClientAdminEndpoint->PublishEvent(DiscoverServersEvent);
	}

	/**
	 * Timeout any stale servers that have stopped responding.
	 */
	void TimeoutDiscovery(const FDateTime& UtcNow)
	{
		int32 TimeoutInSeconds = 5;
		{
			FScopeLock DiscoveryDataLock(&DiscoveryEventDataCS);
			TimeoutInSeconds = DiscoveryEventData.DiscoveryTimeoutSeconds;
		}
		
		const FTimespan DiscoveryTimeoutSpan = FTimespan(0, 0, TimeoutInSeconds);
		if (DiscoveryTimeoutSpan.IsZero())
		{
			return;
		}

		FScopeLock KnownServersLock(&KnownServersCS);
		bool TimeoutOccurred = false;
		for (auto It = KnownServers.CreateIterator(); It; ++It)
		{
			if (It->Value.LastDiscoveryTime + DiscoveryTimeoutSpan <= UtcNow)
			{
				TimeoutOccurred = true;
				UE_LOGF(LogConcert, Display, "Server %ls lost.", *It->Value.ServerInfo.ServerName);
				It.RemoveCurrent();
				continue;
			}
		}
	
		if (TimeoutOccurred)
		{
			SetServersUpdatedFlag();
		}
	}

	/** Simple struct to store the state of discovered servers.  Each server gets a last known time so that we can expire stale servers. */
	struct FKnownServer
	{
		/** Last time we saw this server. */
		FDateTime LastDiscoveryTime;

		/** Server specific information: name, version, etc. */
		FConcertServerInfo ServerInfo;
	};

	
	/** Mutex to protect access to known servers map. */
	mutable FCriticalSection KnownServersCS;

	/** Map of discovered Concert Servers */
	TMap<FGuid, FKnownServer> KnownServers;

	/** Atomics for state tracking and external access to the runnable. */
	std::atomic<bool> bIsRunning = false;
	std::atomic<bool> bStopRequested = false;

	/**
	 * When this is true the servers list was updated.  The discovery ticker in ConcertClient will poll this periodically to determine if
	 * a server updated delegate should be sent.
	 */
	std::atomic<bool> bServersUpdated = false;

	/** Shared pointer access to the local endpoint for this client. */
	TSharedPtr<IConcertLocalEndpoint> ClientAdminEndpoint;

	/** Mutex for discovery event data below. */
	mutable FCriticalSection DiscoveryEventDataCS;

	/** Discovery event data is data that is required to send discovery events.  This data exists on the settings object but it is copied into the thread for message sending. */
	struct FDiscoveryEventData
	{
		/** Populated and updated by the main thread during discovery. */
		FString CurrentRole;

		/** Authentication key for accessing the server. This is established at app startup. */
		FString CurrentAuthKey;

		/** The timespace at which discovered Multi-User server are considered stale if they haven't answered back. */
		int32 DiscoveryTimeoutSeconds = 5;
	};

	/** Data we need to pass long in the discovery event message we send to the server. */
	FDiscoveryEventData DiscoveryEventData;
		
	TUniquePtr<FRunnableThread> Thread;
};


FConcertClient::FConcertClient(const FString& InRole, const TSharedPtr<IConcertEndpointProvider>& InEndpointProvider)
	: Role(InRole)
	, Paths(InRole)
	, EndpointProvider(InEndpointProvider)
	, DiscoveryCount(0)
	, bClientSessionPendingDestroy(false)
{

}

FConcertClient::~FConcertClient()
{
	// if the ClientAdminEndpoint is valid, Shutdown wasn't called
	check(!ClientAdminEndpoint.IsValid());
}

const FString& FConcertClient::GetRole() const
{
	return Role;
}

void FConcertClient::Configure(const UConcertClientConfig* InSettings)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	ClientInfo.Initialize();
	check(InSettings != nullptr);
	Settings = TStrongObjectPtr<const UConcertClientConfig>(InSettings);
	// Set the display name from the settings or default to username (i.e. app session owner)
	ClientInfo.DisplayName = Settings->ClientSettings.DisplayName.IsEmpty() ? ClientInfo.UserName : Settings->ClientSettings.DisplayName;
	ClientInfo.AvatarColor = Settings->ClientSettings.AvatarColor;
	ClientInfo.DesktopAvatarActorClass = Settings->ClientSettings.DesktopAvatarActorClass.ToString();
	ClientInfo.VRAvatarActorClass = Settings->ClientSettings.VRAvatarActorClass.ToString();
	ClientInfo.Tags = Settings->ClientSettings.Tags;
}

bool FConcertClient::IsConfigured() const
{
	// if the instance id hasn't been set yet, then Configure wasn't called.
	return Settings && ClientInfo.InstanceInfo.InstanceId.IsValid();
}

const UConcertClientConfig* FConcertClient::GetConfiguration() const
{
	return Settings.Get();
}

const FConcertClientInfo& FConcertClient::GetClientInfo() const
{
	// NOTE: The 'ClientSession->ClientInfo'can dynamically be updated during the session, e.g. avatar class can change to reflect the client state: PIE, VR, etc.
	//       The 'this->ClientInfo' member change when the Configure() function is called (when the settings panel is changed).
	//       IConcertSessionClient and IConcertClient must return the same client info to be consistent.
	return ClientSession ? ClientSession->GetLocalClientInfo() : ClientInfo;
}

bool FConcertClient::IsStarted() const
{
	return ClientAdminEndpoint.IsValid();
}

void FConcertClient::Startup()
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);
	check(IsConfigured());
	if (!ClientAdminEndpoint.IsValid() && EndpointProvider.IsValid())
	{
		// Create the client administration endpoint
		ClientAdminEndpoint = EndpointProvider->CreateLocalEndpoint(TEXT("Admin"), Settings->EndpointSettings, [this](const FConcertEndpointContext& Context)
		{
			return ConcertUtil::CreateLogger(Context, [this](const FConcertLog& Log)
			{
				ConcertTransportEvents::OnConcertClientLogEvent().Broadcast(*this, Log);
			});
		});
	}

	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertClient::OnEndFrame);
}

void FConcertClient::Shutdown()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	// Remove Auto Connection routine, if any
	AutoConnection.Reset();

	while (IsDiscoveryEnabled())
	{
		StopDiscovery();
	}
	ClientAdminEndpoint.Reset();

	if (ClientSession.IsValid())
	{
		ClientSession->Disconnect();
		OnSessionShutdownDelegate.Broadcast(ClientSession.ToSharedRef());
		ClientSession->Shutdown();
		ClientSession.Reset();
	}

	// Clear the working directory for this instance
	ConcertUtil::DeleteDirectoryTree(*Paths.GetWorkingDir());
}

bool FConcertClient::IsDiscoveryEnabled() const
{
	return DiscoveryCount > 0;
}

void FConcertClient::StartDiscovery()
{
	++DiscoveryCount;
	if (ClientAdminEndpoint.IsValid() && !DiscoveryTick.IsValid())
	{
		ServerDiscoveryThread = MakeUnique<FConcertClientServerDiscoveryThread>(ClientAdminEndpoint, Role, Settings->ClientSettings.ClientAuthenticationKey, Settings->ClientSettings.DiscoveryTimeoutSeconds);
		
		DiscoveryTick = FTSTicker::GetCoreTicker().AddTicker(TEXT("Discovery"), 1, [this](float DeltaSeconds) {
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FConcertClient_Discovery_Tick);

			if (ServerDiscoveryThread)
			{
				ServerDiscoveryThread->SetDiscoveryEventInformation(Role, Settings->ClientSettings.ClientAuthenticationKey, Settings->ClientSettings.DiscoveryTimeoutSeconds);
				if (ServerDiscoveryThread->HasServerListChanged())
				{
					ServerDiscoveryThread->ResetServerListChange();
					ServersUpdatedDelegate.Broadcast();
				}
			}

			return true;
		});
	}
}

void FConcertClient::StopDiscovery()
{
	check(IsDiscoveryEnabled());
	--DiscoveryCount;
	if (DiscoveryCount > 0)
	{
		return;
	}

	if (ServerDiscoveryThread)
	{
		ServerDiscoveryThread->Stop();		
	}
	
	if (DiscoveryTick.IsValid())
	{
		FTSTicker::RemoveTicker(DiscoveryTick);
		DiscoveryTick.Reset();
	}
}

bool FConcertClient::CanAutoConnect() const
{
	return IsConfigured() && Settings && !Settings->DefaultServerURL.IsEmpty() && !Settings->DefaultSessionName.IsEmpty();
}

bool FConcertClient::IsAutoConnecting() const
{
	return AutoConnection.IsValid();
}

void FConcertClient::StartAutoConnect()
{
	check(IsStarted());

	if (AutoConnection.IsValid())
	{
		return;
	}

	if (CanAutoConnect())
	{
		// Cancel the in-progress connection process if any, (even if it was connecting to the default session) to connect to the default session instead.
		PendingConnection.Reset();
		AutoConnection = MakeUnique<FConcertAutoConnection>(this, Settings.Get());
	}
}

void FConcertClient::StopAutoConnect()
{
	if (IsAutoConnecting())
	{
		// Cancel the in-progress auto-connection process (if any).
		PendingConnection.Reset();
	}
	AutoConnection.Reset();
}

FConcertConnectionError FConcertClient::GetLastConnectionError() const
{
	return LastConnectionError;
}

TArray<FConcertServerInfo> FConcertClient::GetKnownServers() const
{
	if (ServerDiscoveryThread)
	{
		return ServerDiscoveryThread->GetKnownServers();
	}
	return {};
}

FSimpleMulticastDelegate& FConcertClient::OnKnownServersUpdated()
{
	return ServersUpdatedDelegate;
}

FOnConcertClientSessionStartupOrShutdown& FConcertClient::OnSessionStartup()
{
	return OnSessionStartupDelegate;
}

FOnConcertClientSessionStartupOrShutdown& FConcertClient::OnSessionShutdown()
{
	return OnSessionShutdownDelegate;
}

FOnConcertClientSessionGetPreConnectionTasks& FConcertClient::OnGetPreConnectionTasks()
{
	return OnGetPreConnectionTasksDelegate;
}

FOnConcertClientSessionConnectionChanged& FConcertClient::OnSessionConnectionChanged()
{
	return OnSessionConnectionChangedDelegate;
}

void FConcertClient::SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService)
{
	FileSharingService = MoveTemp(InFileSharingService);

	if (FileSharingService)
	{
		// Start with the FileSharingService disabled.  Per session, we determine if it should be on.
		FileSharingService->SetEnabled(false);
	}
}

TSharedPtr<IConcertFileSharingService> FConcertClient::GetFileSharingService() const
{
	return FileSharingService;
}

EConcertConnectionStatus FConcertClient::GetSessionConnectionStatus() const
{
	return ClientSession.IsValid() ? ClientSession->GetConnectionStatus() : EConcertConnectionStatus::Disconnected;
}

TFuture<EConcertResponseCode> FConcertClient::CreateSession(const FGuid& ServerAdminEndpointId, const FConcertCreateSessionArgs& CreateSessionArgs)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	// We don't want the client to get automatically reconnected to it's default session if something wrong happens
	AutoConnection.Reset();
	return InternalCreateSession(ServerAdminEndpointId, CreateSessionArgs);
}

TFuture<EConcertResponseCode> FConcertClient::JoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	// We don't want the client to get automatically reconnected to it's default session if something wrong happens
	AutoConnection.Reset();
	return InternalJoinSession(ServerAdminEndpointId, SessionId);
}

TFuture<EConcertResponseCode> FConcertClient::RestoreSession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& RestoreSessionArgs)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	// We don't want the client to get automatically reconnected to the default session if something wrong happens
	if (RestoreSessionArgs.bAutoConnect)
	{
		AutoConnection.Reset();
	}
	return InternalCopySession(ServerAdminEndpointId, RestoreSessionArgs, /*bRestoreOnlyConstraint*/true);
}

TFuture<EConcertResponseCode> FConcertClient::CopySession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& CopySessionArgs)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);
	
	// We don't want the client to get automatically reconnected to the default session if the copy/connect fails.
	if (CopySessionArgs.bAutoConnect)
	{
		AutoConnection.Reset();
	}
	return InternalCopySession(ServerAdminEndpointId, CopySessionArgs, /*bRestoreOnlyConstraint*/false);
}

TFuture<EConcertResponseCode> FConcertClient::ArchiveSession(const FGuid& ServerAdminEndpointId, const FConcertArchiveSessionArgs& ArchiveSessionArgs)
{
	FConcertAdmin_ArchiveSessionRequest ArchiveSessionRequest;
	ArchiveSessionRequest.SessionId = ArchiveSessionArgs.SessionId;
	ArchiveSessionRequest.ArchiveNameOverride = ArchiveSessionArgs.ArchiveNameOverride;
	ArchiveSessionRequest.SessionFilter = ArchiveSessionArgs.SessionFilter;

	// Fill the information for the client identification
	ArchiveSessionRequest.UserName = ClientInfo.UserName;
	ArchiveSessionRequest.DeviceName = ClientInfo.DeviceName;

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bIsHeadless = Settings->bIsHeadless;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("ArchivingSession", "Archiving Session...");
	NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();

	FAsyncTaskNotification Notification(NotificationConfig);

	return ClientAdminEndpoint->SendRequest<FConcertAdmin_ArchiveSessionRequest, FConcertAdmin_ArchiveSessionResponse>(ArchiveSessionRequest, ServerAdminEndpointId)
		.Next([this, Notification = MoveTemp(Notification)](const FConcertAdmin_ArchiveSessionResponse& RequestResponse) mutable
		{
			if (RequestResponse.ResponseCode == EConcertResponseCode::Success)
			{
				Notification.SetComplete(FText::Format(LOCTEXT("ArchivedSessionFmt", "Archived Session '{0}' as '{1}"), FText::FromString(RequestResponse.SessionName), FText::FromString(RequestResponse.ArchiveName)), FText(), true);
			}
			else
			{
				Notification.SetComplete(FText::Format(LOCTEXT("FailedToArchiveSessionFmt", "Failed to Archive Session '{0}'"), FText::FromString(RequestResponse.SessionName)), RequestResponse.Reason, false);
			}
			return RequestResponse.ResponseCode;
		});
}

TFuture<EConcertResponseCode> FConcertClient::RenameSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName)
{
	FConcertAdmin_RenameSessionRequest RenameSessionRequest;
	RenameSessionRequest.SessionId = SessionId;
	RenameSessionRequest.NewName = NewName;

	// Fill the information for the client identification
	RenameSessionRequest.UserName = ClientInfo.UserName;
	RenameSessionRequest.DeviceName = ClientInfo.DeviceName;

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bIsHeadless = Settings->bIsHeadless;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("RenamingSession", "Renaming Session...");
	NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();

	FAsyncTaskNotification Notification(NotificationConfig);

	return ClientAdminEndpoint->SendRequest<FConcertAdmin_RenameSessionRequest, FConcertAdmin_RenameSessionResponse>(RenameSessionRequest, ServerAdminEndpointId)
		.Next([this, NewName, Notification = MoveTemp(Notification)](const FConcertAdmin_RenameSessionResponse& RequestResponse) mutable
		{
			if (RequestResponse.ResponseCode == EConcertResponseCode::Success)
			{
				Notification.SetComplete(FText::Format(LOCTEXT("RenamedSessionFmt", "Renamed Session '{0}' as '{1}'"), FText::AsCultureInvariant(RequestResponse.OldName), FText::AsCultureInvariant(NewName)), FText(), true);
			}
			else
			{
				Notification.SetComplete(FText::Format(LOCTEXT("FailedToRenameSessionFmt", "Failed to Rename Session '{0}' as '{1}'"), FText::AsCultureInvariant(RequestResponse.OldName), FText::AsCultureInvariant(NewName)), RequestResponse.Reason, false);
			}
			return RequestResponse.ResponseCode;
		});
}

TFuture<EConcertResponseCode> FConcertClient::DeleteSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId)
{
	FConcertAdmin_DeleteSessionRequest DeleteSessionRequest;
	DeleteSessionRequest.SessionId = SessionId;

	// Fill the information for the client identification
	DeleteSessionRequest.UserName = ClientInfo.UserName;
	DeleteSessionRequest.DeviceName = ClientInfo.DeviceName;

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bIsHeadless = Settings->bIsHeadless;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("DeletingSession", "Deleting Session...");
	NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();

	FAsyncTaskNotification Notification(NotificationConfig);

	return ClientAdminEndpoint->SendRequest<FConcertAdmin_DeleteSessionRequest, FConcertAdmin_DeleteSessionResponse>(DeleteSessionRequest, ServerAdminEndpointId)
		.Next([this, Notification = MoveTemp(Notification)](const FConcertAdmin_DeleteSessionResponse& RequestResponse) mutable
		{
			if (RequestResponse.ResponseCode == EConcertResponseCode::Success)
			{
				Notification.SetComplete(FText::Format(LOCTEXT("DeletedSessionFmt", "Deleted Session '{0}'"), FText::FromString(RequestResponse.SessionName)), FText(), true);
			}
			else
			{
				Notification.SetComplete(FText::Format(LOCTEXT("FailedToDeleteSessionFmt", "Failed to Delete Session '{0}'"), FText::FromString(RequestResponse.SessionName)), RequestResponse.Reason, false);
			}
			return RequestResponse.ResponseCode;
		});
}

TFuture<FConcertAdmin_BatchDeleteSessionResponse> FConcertClient::BatchDeleteSessions(const FGuid& ServerAdminEndpointId, const FConcertBatchDeleteSessionsArgs& BatchDeletionArgs)
{
	FConcertAdmin_BatchDeleteSessionRequest DeleteSessionRequest;
	DeleteSessionRequest.SessionIds = BatchDeletionArgs.SessionIds;
	DeleteSessionRequest.Flags = BatchDeletionArgs.Flags;

	// Fill the information for the client identification
	DeleteSessionRequest.UserName = ClientInfo.UserName;
	DeleteSessionRequest.DeviceName = ClientInfo.DeviceName;

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bIsHeadless = Settings->bIsHeadless;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("DeletingSessions", "Deleting Sessions...");
	NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();

	FAsyncTaskNotification Notification(NotificationConfig);

	return ClientAdminEndpoint->SendRequest<FConcertAdmin_BatchDeleteSessionRequest, FConcertAdmin_BatchDeleteSessionResponse>(DeleteSessionRequest, ServerAdminEndpointId)
		.Next([this, NumRequested = DeleteSessionRequest.SessionIds.Num(), Notification = MoveTemp(Notification)](const FConcertAdmin_BatchDeleteSessionResponse& RequestResponse) mutable
		{
			if (RequestResponse.ResponseCode == EConcertResponseCode::Success)
			{
				const bool bDeletedAll = RequestResponse.DeletedItems.Num() == NumRequested;
				const FText Message = bDeletedAll
					? FText::Format(LOCTEXT("DeletedSessionsFmt.All", "Deleted {0} Sessions"), RequestResponse.DeletedItems.Num())
					: FText::Format(LOCTEXT("DeletedSessionsFmt.Some", "Deleted {0} of {1} Sessions"), RequestResponse.DeletedItems.Num(), NumRequested);
				const FText ProgressText = bDeletedAll
					? FText::GetEmpty()
					: [&RequestResponse]()
					{
						TArray<FString> SessionNames;
						Algo::Transform(RequestResponse.NotOwnedByClient, SessionNames, [](const FDeletedSessionInfo& Skipped){ return Skipped.SessionName; });
						return FText::FromString(FString::Join(SessionNames, TEXT(", ")));
					}();
				Notification.SetComplete(Message, ProgressText, true);
			}
			else
			{
				Notification.SetComplete(LOCTEXT("FailedToDeleteSessionsFmt", "Failed to Delete Sessions"), RequestResponse.Reason, false);
			}

			return RequestResponse;
		});
}

void FConcertClient::DisconnectSession()
{
	// We don't want the client to get automatically reconnected to it's default session
	AutoConnection.Reset();
	InternalDisconnectSession();

	// If async connection tasks were in-flight, cancel them.
	PendingConnection.Reset();
}

EConcertSendReceiveState FConcertClient::GetSendReceiveState() const
{
	if (ClientSession.IsValid())
	{
		return ClientSession->GetSendReceiveState();
	}
	return EConcertSendReceiveState::Default;
}

void FConcertClient::SetSendReceiveState(EConcertSendReceiveState InSendReceiveState)
{
	if (ClientSession.IsValid())
	{
		ClientSession->SetSendReceiveState(InSendReceiveState);
	}
}

bool FConcertClient::IsOwnerOf(const FConcertSessionInfo& InSessionInfo) const
{
	return ClientInfo.UserName == InSessionInfo.OwnerUserName && ClientInfo.DeviceName == InSessionInfo.OwnerDeviceName;
}

TSharedPtr<IConcertClientSession> FConcertClient::GetCurrentSession() const
{
	return ClientSession;
}

TFuture<FConcertAdmin_MountSessionRepositoryResponse> FConcertClient::MountSessionRepository(const FGuid& ServerAdminEndpointId, const FString& RepositoryRootDir, const FGuid& RepositoryId, bool bCreateIfNotExist, bool bAsDefault) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_MountSessionRepositoryRequest MountRepositoryRequest;
	MountRepositoryRequest.RepositoryId = RepositoryId;
	MountRepositoryRequest.RepositoryRootDir = RepositoryRootDir;
	MountRepositoryRequest.bCreateIfNotExist = bCreateIfNotExist;
	MountRepositoryRequest.bAsServerDefault = bAsDefault;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_MountSessionRepositoryRequest, FConcertAdmin_MountSessionRepositoryResponse>(MountRepositoryRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetSessionRepositoriesResponse> FConcertClient::GetSessionRepositories(const FGuid& ServerAdminEndpointId) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetSessionRepositoriesRequest GetRepositoryRequest;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetSessionRepositoriesRequest, FConcertAdmin_GetSessionRepositoriesResponse>(GetRepositoryRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_DropSessionRepositoriesResponse> FConcertClient::DropSessionRepositories(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& RepositoryIds) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_DropSessionRepositoriesRequest DropRepositoryRequest;
	DropRepositoryRequest.RepositoryIds = RepositoryIds;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_DropSessionRepositoriesRequest, FConcertAdmin_DropSessionRepositoriesResponse>(DropRepositoryRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetAllSessionsResponse> FConcertClient::GetServerSessions(const FGuid& ServerAdminEndpointId) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetAllSessionsRequest GetSessionsRequest = FConcertAdmin_GetAllSessionsRequest();
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetAllSessionsRequest, FConcertAdmin_GetAllSessionsResponse>(GetSessionsRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetSessionsResponse> FConcertClient::GetLiveSessions(const FGuid& ServerAdminEndpointId) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetLiveSessionsRequest GetLiveSessionsRequest;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetLiveSessionsRequest, FConcertAdmin_GetSessionsResponse>(GetLiveSessionsRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetSessionsResponse> FConcertClient::GetArchivedSessions(const FGuid& ServerAdminEndpointId) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetArchivedSessionsRequest GetArchivedSessionsRequest;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetArchivedSessionsRequest, FConcertAdmin_GetSessionsResponse>(GetArchivedSessionsRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetSessionClientsResponse> FConcertClient::GetSessionClients(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetSessionClientsRequest GetSessionClientsRequest;
	GetSessionClientsRequest.SessionId = SessionId;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetSessionClientsRequest, FConcertAdmin_GetSessionClientsResponse>(GetSessionClientsRequest, ServerAdminEndpointId);
}

TFuture<FConcertAdmin_GetSessionActivitiesResponse> FConcertClient::GetSessionActivities(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, bool bIncludeDetails) const
{
	LLM_SCOPE_BYTAG(Concert_ConcertClient);

	FConcertAdmin_GetSessionActivitiesRequest GetSessionActivitiesRequest;
	GetSessionActivitiesRequest.SessionId = SessionId;
	GetSessionActivitiesRequest.FromActivityId = FromActivityId;
	GetSessionActivitiesRequest.ActivityCount = ActivityCount;
	GetSessionActivitiesRequest.bIncludeDetails = bIncludeDetails;
	return ClientAdminEndpoint->SendRequest<FConcertAdmin_GetSessionActivitiesRequest, FConcertAdmin_GetSessionActivitiesResponse>(GetSessionActivitiesRequest, ServerAdminEndpointId);
}

TFuture<EConcertResponseCode> FConcertClient::InternalCreateSession(const FGuid& ServerAdminEndpointId, const FConcertCreateSessionArgs& CreateSessionArgs, TUniquePtr<FAsyncTaskNotification> OngoingNotification)
{
	// Reset last connection error
	LastConnectionError = FConcertConnectionError();

	// Cancel any pending connection (will be aborted)
	PendingConnection.Reset();

	// Build the tasks to execute
	TArray<TUniquePtr<IConcertClientConnectionTask>> ConnectionTasks;

	// Collect pre-connection tasks
	OnGetPreConnectionTasksDelegate.Broadcast(*this, ConnectionTasks);

	// Create session task
	{
		// Fill create session request;
		FConcertAdmin_CreateSessionRequest CreateSessionRequest;
		CreateSessionRequest.SessionName = CreateSessionArgs.SessionName;
		CreateSessionRequest.OwnerClientInfo = ClientInfo;
		CreateSessionRequest.VersionInfo.Initialize(GetConfiguration()->ClientSettings.bSupportMixedBuildTypes);
	
		// Session settings
		CreateSessionRequest.SessionSettings.Initialize();
		CreateSessionRequest.SessionSettings.ArchiveNameOverride = CreateSessionArgs.ArchiveNameOverride;

		if (GetConfiguration()->ClientSettings.bEnableFileSharing)
		{
			CreateSessionRequest.FileShare = UE::ConcertClient::Private::MakeFileSharingChallenge(FileSharingService, GetConfiguration()->ClientSettings.CloudFileSharingPath);
		}

		ConnectionTasks.Emplace(MakeUnique<FConcertClientCreateSessionTask>(this, MoveTemp(CreateSessionRequest), ServerAdminEndpointId));
	}

	// Pending connection config
	const FText SessionNameText = FText::FromString(CreateSessionArgs.SessionName);
	FConcertPendingConnection::FConfig PendingConnectionConfig;
	PendingConnectionConfig.PendingTitleText = FText::Format(LOCTEXT("CreatingSessionFmt", "Creating Session '{0}'..."), SessionNameText);
	PendingConnectionConfig.SuccessTitleText = FText::Format(LOCTEXT("CreatedSessionFmt", "Created Session '{0}'"), SessionNameText);
	PendingConnectionConfig.FailureTitleText = FText::Format(LOCTEXT("FailedToCreateSessionFmt", "Failed to Create Session '{0}'"), SessionNameText);
	PendingConnectionConfig.KeepNotificationOpenOnError = TAttribute<bool>::Create([KeepErrorOnScreen = !Settings->bRetryAutoConnectOnError] { return KeepErrorOnScreen; }); // If the connection fails and retry is off, we can keep the error on screen.
	PendingConnectionConfig.bIsAutoConnection = AutoConnection.IsValid();

	// Kick off a pending connection to execute the tasks
	PendingConnection = MakeShared<FConcertPendingConnection>(this, PendingConnectionConfig);
	return PendingConnection->Execute(MoveTemp(ConnectionTasks), MoveTemp(OngoingNotification));
}

TFuture<EConcertResponseCode> FConcertClient::InternalJoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, TUniquePtr<FAsyncTaskNotification> OngoingNotification)
{
	// Reset last connection error
	LastConnectionError = FConcertConnectionError();

	// Cancel any pending connection (will be aborted)
	PendingConnection.Reset();

	// Build the tasks to execute
	TArray<TUniquePtr<IConcertClientConnectionTask>> ConnectionTasks;

	// Collect pre-connection tasks
	OnGetPreConnectionTasksDelegate.Broadcast(*this, ConnectionTasks);

	// Will be filled in with the resolved session name during the join process
	TSharedRef<FText> ResolvedSessionName = MakeShared<FText>();

	// Find session task
	{
		// Fill find session request
		FConcertAdmin_FindSessionRequest FindSessionRequest;
		FindSessionRequest.SessionId = SessionId;
		FindSessionRequest.OwnerClientInfo = ClientInfo;
		FindSessionRequest.VersionInfo.Initialize(GetConfiguration()->ClientSettings.bSupportMixedBuildTypes);

		// Session settings
		FindSessionRequest.SessionSettings.Initialize();

		if (GetConfiguration()->ClientSettings.bEnableFileSharing)
		{
			FindSessionRequest.FileShare = UE::ConcertClient::Private::MakeFileSharingChallenge(FileSharingService, GetConfiguration()->ClientSettings.CloudFileSharingPath);
		}
		
		ConnectionTasks.Emplace(MakeUnique<FConcertClientJoinSessionTask>(this, MoveTemp(FindSessionRequest), ServerAdminEndpointId, ResolvedSessionName));
	}

	// Pending connection config
	FConcertPendingConnection::FConfig PendingConnectionConfig;
	PendingConnectionConfig.PendingTitleText = LOCTEXT("JoiningSession", "Joining Session...");
	PendingConnectionConfig.SuccessTitleText.Bind(TAttribute<FText>::FGetter::CreateLambda([ResolvedSessionName]() { return FText::Format(LOCTEXT("JoinedSessionFmt", "Joined Session '{0}'"), *ResolvedSessionName); }));
	PendingConnectionConfig.FailureTitleText.Bind(TAttribute<FText>::FGetter::CreateLambda([ResolvedSessionName]() { return ResolvedSessionName->IsEmpty() ? LOCTEXT("FailedToJoinSession", "Failed to Join Session") : FText::Format(LOCTEXT("FailedToJoinSessionFmt", "Failed to Join Session '{0}'"), *ResolvedSessionName); }));
	PendingConnectionConfig.KeepNotificationOpenOnError = TAttribute<bool>::Create([KeepErrorOnScreen = !Settings->bRetryAutoConnectOnError] { return KeepErrorOnScreen; }); // If the connection fails and retry is off, we can keep the error on screen.
	PendingConnectionConfig.bIsAutoConnection = AutoConnection.IsValid();

	// Kick off a pending connection to execute the tasks
	PendingConnection = MakeShared<FConcertPendingConnection>(this, PendingConnectionConfig);
	return PendingConnection->Execute(MoveTemp(ConnectionTasks), MoveTemp(OngoingNotification));
}

TFuture<EConcertResponseCode> FConcertClient::InternalCopySession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& CopySessionArgs, bool bRestoreOnlyConstraint, TUniquePtr<FAsyncTaskNotification> OngoingNotification)
{
	FConcertAdmin_CopySessionRequest CopySessionRequest;
	CopySessionRequest.SessionId = CopySessionArgs.SessionId;
	CopySessionRequest.SessionName = CopySessionArgs.SessionName;
	CopySessionRequest.SessionFilter = CopySessionArgs.SessionFilter;
	CopySessionRequest.bRestoreOnly = bRestoreOnlyConstraint;
	CopySessionRequest.OwnerClientInfo = ClientInfo;
	CopySessionRequest.VersionInfo.Initialize(GetConfiguration()->ClientSettings.bSupportMixedBuildTypes);

	// Session settings
	CopySessionRequest.SessionSettings.Initialize();
	CopySessionRequest.SessionSettings.ArchiveNameOverride = CopySessionArgs.ArchiveNameOverride;

	TUniquePtr<FAsyncTaskNotification> Notification = MoveTemp(OngoingNotification);
	if (!Notification.IsValid())
	{
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.bIsHeadless = Settings->bIsHeadless;
		NotificationConfig.bKeepOpenOnFailure = true;
		NotificationConfig.TitleText = bRestoreOnlyConstraint ? LOCTEXT("RestoringSession", "Restoring Session...") : LOCTEXT("CopyingSession", "Copying Session...");
		NotificationConfig.LogCategory = ConcertUtil::GetLogConcertPtr();
		Notification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
	}

	return ClientAdminEndpoint->SendRequest<FConcertAdmin_CopySessionRequest, FConcertAdmin_SessionInfoResponse>(CopySessionRequest, ServerAdminEndpointId)
		.Next([this, Notification = MoveTemp(Notification), bAutoConnect = CopySessionArgs.bAutoConnect, bRestoreOnly = bRestoreOnlyConstraint](const FConcertAdmin_SessionInfoResponse& RequestResponse) mutable
	{
		if (RequestResponse.ResponseCode == EConcertResponseCode::Success)
		{
			if (bAutoConnect)
			{
				InternalJoinSession(RequestResponse.ConcertEndpointId, RequestResponse.SessionInfo.SessionId, MoveTemp(Notification));
			}
			else
			{
				Notification->SetComplete(FText::Format(bRestoreOnly ? LOCTEXT("RestoreSessionFmt", "Restored Session '{0}'") : LOCTEXT("CopySessionFmt", "Copied Session '{0}'"), FText::FromString(RequestResponse.SessionInfo.SessionName)), FText(), true);
			}
		}
		else
		{
			Notification->SetComplete(bRestoreOnly ? LOCTEXT("FailedToRestoreSession", "Failed to Restore Session") : LOCTEXT("FailedToCopySession", "Failed to Copy Session"), RequestResponse.Reason, false);
		}
		return RequestResponse.ResponseCode;
	});
}

void FConcertClient::InternalDisconnectSession()
{
	if (ClientSession.IsValid())
	{
		ClientSession->Disconnect();
		OnSessionShutdownDelegate.Broadcast(ClientSession.ToSharedRef());
		ClientSession->Shutdown();
		ClientSession.Reset();

		// Turn off the file sharing service.
		if (FileSharingService)
		{
			FileSharingService->SetEnabled(false);
		}
	}

	bClientSessionPendingDestroy = false;
}

void FConcertClient::SetLastConnectionError(FConcertConnectionError LastError)
{
	LastConnectionError = MoveTemp(LastError);
}

void FConcertClient::OnEndFrame()
{
	if (bClientSessionPendingDestroy)
	{
		InternalDisconnectSession();
		bClientSessionPendingDestroy = false;
	}
}

TFuture<EConcertResponseCode> FConcertClient::CreateClientSession(const FConcertSessionInfo& SessionInfo)
{
	check(SessionInfo.SessionId.IsValid() && !SessionInfo.SessionName.IsEmpty());

	InternalDisconnectSession();
	ClientSession = MakeShared<FConcertClientSession>(
		SessionInfo,
		ClientInfo,
		Settings->ClientSettings,
		EndpointProvider->CreateLocalEndpoint(SessionInfo.SessionName, Settings->EndpointSettings, [this](const FConcertEndpointContext& Context) {
			return ConcertUtil::CreateLogger(Context, [this](const FConcertLog& Log)
			{ ConcertTransportEvents::OnConcertClientLogEvent().Broadcast(*this, Log);
			});
		}),
		Paths.GetSessionWorkingDir(SessionInfo.SessionId));

	OnSessionStartupDelegate.Broadcast(ClientSession.ToSharedRef());
	ClientSession->OnConnectionChanged().AddRaw(this, &FConcertClient::HandleSessionConnectionChanged);
	ClientSession->Startup();
	ClientSession->Connect();

	// Promise the caller to tell it once the client connection state is known (connected or not).
	check(!ConnectionPromise.IsValid()); // InternalDisconnect() triggers a disconnnect and this will release of the promise.
	ConnectionPromise = MakeUnique<TPromise<EConcertResponseCode>>();
	
	return ConnectionPromise->GetFuture();
}

void FConcertClient::HandleSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus Status)
{
	// If this session disconnected, make sure we fully destroy it at the end of the frame
	if (Status == EConcertConnectionStatus::Disconnected)
	{
		bClientSessionPendingDestroy = true;
		if (ConnectionPromise) // Tried to connect, but did not succeed? (Like an endpoint timeout/client shutting down while connecting)
		{
			ConnectionPromise->SetValue(EConcertResponseCode::Failed);
			ConnectionPromise.Reset();
		}
	}
	else if (Status == EConcertConnectionStatus::Connected)
	{
		check(ConnectionPromise.IsValid());
		ConnectionPromise->SetValue(EConcertResponseCode::Success);
		ConnectionPromise.Reset();
	}

	OnSessionConnectionChangedDelegate.Broadcast(InSession, Status);
}

namespace UE::ConcertClient
{
FOnConcertEvaluateHasRole& VPRoleEvaluator()
{
	static FOnConcertEvaluateHasRole RoleEvaluator;
	return RoleEvaluator;
}
}

#undef LOCTEXT_NAMESPACE
