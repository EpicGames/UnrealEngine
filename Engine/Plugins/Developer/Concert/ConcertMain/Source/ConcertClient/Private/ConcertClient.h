// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertClient.h"
#include "IConcertTransportModule.h"
#include "ConcertSettings.h"

#include "UObject/StrongObjectPtr.h"
#include "Containers/Ticker.h"

class FConcertClientSession;
class FConcertAutoConnection;
class FConcertPendingConnection;
class FConcertClientJoinSessionTask;
class FConcertClientCreateSessionTask;
class FAsyncTaskNotification;
class FConcertClientServerDiscoveryThread;

class FConcertClientPaths
{
public:
	explicit FConcertClientPaths(const FString& InRole);

	/** Get the working directory. This is were the active sessions store their files */
	const FString& GetWorkingDir() const
	{
		return WorkingDir;
	}

	/** Return the working directory for a specific session */
	FString GetSessionWorkingDir(const FGuid& InSessionId) const
	{
		return WorkingDir / InSessionId.ToString();
	}

private:
	/** Get the working directory. This is were the active sessions store their files */
	const FString WorkingDir;
};

/** Implements the Concert client */
class FConcertClient : public IConcertClient
{
public:
	FConcertClient(const FString& InRole, const TSharedPtr<IConcertEndpointProvider>& InEndpointProvider);
	virtual ~FConcertClient();

	virtual const FString& GetRole() const override;

	virtual void Configure(const UConcertClientConfig* InSettings) override;
	virtual bool IsConfigured() const override;
	virtual const UConcertClientConfig* GetConfiguration() const override;
	virtual const FConcertClientInfo& GetClientInfo() const override;

	virtual bool IsStarted() const override;
	virtual void Startup() override;
	virtual void Shutdown() override;

	virtual bool IsDiscoveryEnabled() const override;
	virtual void StartDiscovery() override;
	virtual void StopDiscovery() override;

	virtual bool CanAutoConnect() const override;
	virtual bool IsAutoConnecting() const override;
	virtual void StartAutoConnect() override;
	virtual void StopAutoConnect() override;
	virtual FConcertConnectionError GetLastConnectionError() const override;

	virtual TArray<FConcertServerInfo> GetKnownServers() const override;
	virtual FSimpleMulticastDelegate& OnKnownServersUpdated() override;

	virtual FOnConcertClientSessionStartupOrShutdown& OnSessionStartup() override;
	virtual FOnConcertClientSessionStartupOrShutdown& OnSessionShutdown() override;

	virtual FOnConcertClientSessionGetPreConnectionTasks& OnGetPreConnectionTasks() override;
	virtual FOnConcertClientSessionConnectionChanged& OnSessionConnectionChanged() override;
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) override;
	virtual TSharedPtr<IConcertFileSharingService> GetFileSharingService() const override;		
	
	virtual EConcertConnectionStatus GetSessionConnectionStatus() const override;
	virtual TFuture<EConcertResponseCode> CreateSession(const FGuid& ServerAdminEndpointId, const FConcertCreateSessionArgs& CreateSessionArgs) override;
	virtual TFuture<EConcertResponseCode> JoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) override;
	virtual TFuture<EConcertResponseCode> RestoreSession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& RestoreSessionArgs) override;
	virtual TFuture<EConcertResponseCode> CopySession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& CopySessionArgs) override;
	virtual TFuture<EConcertResponseCode> ArchiveSession(const FGuid& ServerAdminEndpointId, const FConcertArchiveSessionArgs& ArchiveSessionArgs) override;
	virtual TFuture<EConcertResponseCode> RenameSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) override;
	virtual TFuture<EConcertResponseCode> DeleteSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) override;
	virtual TFuture<FConcertAdmin_BatchDeleteSessionResponse> BatchDeleteSessions(const FGuid& ServerAdminEndpointId, const FConcertBatchDeleteSessionsArgs& BatchDeletionArgs) override;
	virtual void DisconnectSession() override;

	virtual EConcertSendReceiveState GetSendReceiveState() const override;
	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) override;

	virtual bool IsOwnerOf(const FConcertSessionInfo& InSessionInfo) const override;
	virtual TSharedPtr<IConcertClientSession> GetCurrentSession() const override;

	virtual TFuture<FConcertAdmin_MountSessionRepositoryResponse> MountSessionRepository(const FGuid& ServerAdminEndpointId, const FString& RepositoryRootDir, const FGuid& RepositoryId, bool bCreateIfNotExist, bool bAsDefault) const override;
	virtual TFuture<FConcertAdmin_GetSessionRepositoriesResponse> GetSessionRepositories(const FGuid& ServerAdminEndpointId) const override;
	virtual TFuture<FConcertAdmin_DropSessionRepositoriesResponse> DropSessionRepositories(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& RepositoryIds) const override;
	virtual TFuture<FConcertAdmin_GetAllSessionsResponse> GetServerSessions(const FGuid& ServerAdminEndpointId) const override;
	virtual TFuture<FConcertAdmin_GetSessionsResponse> GetLiveSessions(const FGuid& ServerAdminEndpointId) const override;
	virtual TFuture<FConcertAdmin_GetSessionsResponse> GetArchivedSessions(const FGuid& ServerAdminEndpointId) const override;
	virtual TFuture<FConcertAdmin_GetSessionClientsResponse> GetSessionClients(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const override;
	virtual TFuture<FConcertAdmin_GetSessionActivitiesResponse> GetSessionActivities(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, bool bIncludeDetails) const override;

private:
	/** internal friend class for auto connection. */
	friend class FConcertAutoConnection;

	/** internal friend classes for pending connections. */
	friend class FConcertPendingConnection;
	friend class FConcertClientJoinSessionTask;
	friend class FConcertClientCreateSessionTask;

	TFuture<EConcertResponseCode> InternalCreateSession(const FGuid& ServerAdminEndpointId, const FConcertCreateSessionArgs& CreateSessionArgs, TUniquePtr<FAsyncTaskNotification> OngoingNotification = nullptr);
	TFuture<EConcertResponseCode> InternalJoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, TUniquePtr<FAsyncTaskNotification> OngoingNotification = nullptr);
	TFuture<EConcertResponseCode> InternalCopySession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& CopySessionArgs, bool bRestoreOnlyConstraint, TUniquePtr<FAsyncTaskNotification> OngoingNotification = nullptr);
	void InternalDisconnectSession();

	/** Set the connection error. */
	void SetLastConnectionError(FConcertConnectionError LastError);

	/** */
	void OnEndFrame();

	/** Create a Concert client session based on the session information provided. The future is set when the client connection is established or has failed. */
	TFuture<EConcertResponseCode> CreateClientSession(const FConcertSessionInfo& SessionInfo);

	/** Internal handler bound to the current session (if any) to propagate via our own OnSessionConnectionChanged delegate */
	void HandleSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus Status);

	/** The role of this client (eg, MultiUser, DisasterRecovery, etc) */
	FString Role;

	/** Cached root paths used by this client */
	FConcertClientPaths Paths;

	/** Endpoint provider */
	TSharedPtr<IConcertEndpointProvider> EndpointProvider;
	
	/** Administration endpoint for the client (i.e. creating, joining sessions) */
	TSharedPtr<IConcertLocalEndpoint> ClientAdminEndpoint;

	/** Shared pointer to the file sharing service to use.  */
	TSharedPtr<IConcertFileSharingService> FileSharingService;

	/** Count of the number of times the discovery has been enabled */
	uint32 DiscoveryCount;

	/** Ticker for interfacing with the discovery thread. */
	FTSTicker::FDelegateHandle DiscoveryTick;

	/** Unique pointer to the discovery thread runnable. */
	TUniquePtr<FConcertClientServerDiscoveryThread> ServerDiscoveryThread;
		
	/** Holds a delegate to be invoked when the server list was updated. */
	FSimpleMulticastDelegate ServersUpdatedDelegate;

	/** Information about this Client */
	FConcertClientInfo ClientInfo;

	/** Delegate for client session startup */
	FOnConcertClientSessionStartupOrShutdown OnSessionStartupDelegate;

	/** Delegate for client session shutdown */
	FOnConcertClientSessionStartupOrShutdown OnSessionShutdownDelegate;

	/** Delegate that is called to get the pre-connection tasks for a client session */
	FOnConcertClientSessionGetPreConnectionTasks OnGetPreConnectionTasksDelegate;

	/** Delegate for when the session connection state changes */
	FOnConcertClientSessionConnectionChanged OnSessionConnectionChangedDelegate;

	/** Pointer to the Concert Session the client is connected to */
	TSharedPtr<FConcertClientSession> ClientSession;

	/** Client settings object we were configured with */
	TStrongObjectPtr<const UConcertClientConfig> Settings;

	/** The last connection attempt error, if any. */
	FConcertConnectionError LastConnectionError;

	/** Holds the auto connection routine, if any. */
	TUniquePtr<FConcertAutoConnection> AutoConnection;

	/** Holds the pending connection routine, if any (shared as it is used as a weak pointer with UI). */
	TSharedPtr<FConcertPendingConnection> PendingConnection;

	/** The promise sets when the connection to the current session is confirmed or infirmed. */
	TUniquePtr<TPromise<EConcertResponseCode>> ConnectionPromise;

	/** True if the client session disconnected this frame and should be fully destroyed at the end of the frame (this is mainly to handle timeouts) */
	bool bClientSessionPendingDestroy;
};
