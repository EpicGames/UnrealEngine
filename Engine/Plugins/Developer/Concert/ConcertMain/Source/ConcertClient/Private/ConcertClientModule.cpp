// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "IConcertTransportModule.h"

#include "IConcertClientModule.h"

#include "ConcertClient.h"

/**
 * Implements the Concert Client module
 */
class FConcertClientModule : public IConcertClientModule
{
public:

	virtual void StartupModule() override
	{
		EndpointProvider = IConcertTransportModule::Get().CreateEndpointProvider();
	}

	virtual void ShutdownModule() override
	{
		EndpointProvider.Reset();
	}

	virtual IConcertClientRef CreateClient(const FString& InRole) override
	{
		return MakeShared<FConcertClient, ESPMode::ThreadSafe>(InRole, EndpointProvider);
	}

private:
	TSharedPtr<IConcertEndpointProvider> EndpointProvider;
};

IMPLEMENT_MODULE(FConcertClientModule, ConcertClient)
