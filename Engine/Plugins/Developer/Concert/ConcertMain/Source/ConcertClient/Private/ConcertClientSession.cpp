// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientSession.h"
#include "ConcertClientSettings.h"
#include "ConcertLogGlobal.h"
#include "ConcertMessages.h"
#include "IConcertClient.h"
#include "IConcertEndpoint.h"

#include "Scratchpad/ConcertScratchpad.h"

#include "EngineAnalytics.h"

#include "Containers/Ticker.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"
#include "UObject/StructOnScope.h"

const FName ConcertClientMessageIdName("ConcertMessageId");

FConcertClientSession::FConcertClientSession(const FConcertSessionInfo& InSessionInfo, const FConcertClientInfo& InClientInfo, const FConcertClientSettings& InSettings, TSharedPtr<IConcertLocalEndpoint> InClientSessionEndpoint, const FString& InSessionDirectory)
	: FConcertSessionCommonImpl(InSessionInfo)
	, ClientInfo(InClientInfo)
	, ConnectionStatus(EConcertConnectionStatus::Disconnected)
	, ClientSessionEndpoint(MoveTemp(InClientSessionEndpoint))
	, LastConnectionTick(0)
	, SessionTickFrequency(0, 0, InSettings.SessionTickFrequencySeconds)
	, SessionDirectory(InSessionDirectory)
{
	SendReceiveState = GetDefaultSendReceiveState();
}

FConcertClientSession::~FConcertClientSession()
{
	// if the SessionTick is valid, Shutdown wasn't called
	check(!SessionTick.IsValid());
}

namespace UE::ConcertClientSession::Private
{
	void SendAnalyticsSessionStarted(const FString& SessionIdString)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SessionID"), SessionIdString));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.MultiUser.SessionStarted"), EventAttributes);
	}
} // namespace UE::ConcertClientSession::Private

void FConcertClientSession::Startup()
{
	// if the session tick isn't valid we haven't started
	if (!SessionTick.IsValid())
	{
		CommonStartup();

		// Register to connection changed event
		ClientSessionEndpoint->OnRemoteEndpointConnectionChanged().AddRaw(this, &FConcertClientSession::HandleRemoteConnectionChanged);

		// Setup the session handlers
		ClientSessionEndpoint->RegisterEventHandler<FConcertSession_JoinSessionResultEvent>(this, &FConcertClientSession::HandleJoinSessionResultEvent);
		ClientSessionEndpoint->RegisterEventHandler<FConcertSession_ClientListUpdatedEvent>(this, &FConcertClientSession::HandleClientListUpdatedEvent);
		ClientSessionEndpoint->RegisterEventHandler<FConcertSession_SessionRenamedEvent>(this, &FConcertClientSession::HandleSessionRenamedEvent);
		ClientSessionEndpoint->RegisterEventHandler<FConcertSession_UpdateClientInfoEvent>(this, &FConcertClientSession::HandleClientInfoUpdatedEvent);

		// Setup Handlers for custom session messages
		ClientSessionEndpoint->RegisterEventHandler<FConcertSession_CustomEvent>(this, &FConcertClientSession::CommonHandleCustomEvent);
		ClientSessionEndpoint->RegisterRequestHandler<FConcertSession_CustomRequest, FConcertSession_CustomResponse>(this, &FConcertClientSession::CommonHandleCustomRequest);

		// Setup the session tick
		SessionTick = FTSTicker::GetCoreTicker().AddTicker(TEXT("ClientSession"), 0, [this](float DeltaSeconds)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FConcertClientSession_Tick);
			const FDateTime UtcNow = FDateTime::UtcNow();
			TickConnection(DeltaSeconds, UtcNow);
			return true;
		});

		FString SessionIdString = SessionInfo.SessionId.ToString();
		UE_LOGF(LogConcert, Display, "Initialized Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionIdString, *SessionInfo.OwnerUserName);
		UE::ConcertClientSession::Private::SendAnalyticsSessionStarted(SessionIdString);
	}
}

void FConcertClientSession::Shutdown()
{
	if (SessionTick.IsValid())
	{
		// Unregister connection changed
		ClientSessionEndpoint->OnRemoteEndpointConnectionChanged().RemoveAll(this);

		// Unregister the session handlers
		ClientSessionEndpoint->UnregisterEventHandler<FConcertSession_JoinSessionResultEvent>();
		ClientSessionEndpoint->UnregisterEventHandler<FConcertSession_ClientListUpdatedEvent>();
		ClientSessionEndpoint->UnregisterEventHandler<FConcertSession_SessionRenamedEvent>();
		ClientSessionEndpoint->UnregisterEventHandler<FConcertSession_UpdateClientInfoEvent>();

		// Unregister handlers for the custom session messages
		ClientSessionEndpoint->UnregisterEventHandler<FConcertSession_CustomEvent>();
		ClientSessionEndpoint->UnregisterRequestHandler<FConcertSession_CustomRequest>();

		// Unregister the session tick
		FTSTicker::RemoveTicker(SessionTick);
		SessionTick.Reset();

		CommonShutdown();

		UE_LOGF(LogConcert, Display, "Shutdown Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}
}

void FConcertClientSession::Connect()
{
	if (ConnectionStatus == EConcertConnectionStatus::Disconnected)
	{
		// Start connection handshake with server session
		ConnectionStatus = EConcertConnectionStatus::Connecting;
		OnConnectionChangedDelegate.Broadcast(*this, ConnectionStatus);
		UE_LOGF(LogConcert, Display, "Connecting to Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
		SendConnectionRequest();
	}
}

void FConcertClientSession::Disconnect()
{
	if (ConnectionStatus != EConcertConnectionStatus::Disconnected)
	{
		if (ConnectionStatus == EConcertConnectionStatus::Connected)
		{
			SendDisconnection();
		}
		ConnectionStatus = EConcertConnectionStatus::Disconnected;
		UpdateSessionClients(TArray<FConcertSessionClientInfo>());

		// Send Disconnected event
		OnConnectionChangedDelegate.Broadcast(*this, ConnectionStatus);

		UE_LOGF(LogConcert, Display, "Disconnected from Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}
}

EConcertSendReceiveState FConcertClientSession::GetDefaultSendReceiveState()
{
	FOnConcertEvaluateHasRole& OnEvaluateRole = UE::ConcertClient::VPRoleEvaluator();
	if (!OnEvaluateRole.IsBound())
	{
		return EConcertSendReceiveState::Default;
	}

	const UConcertClientConfig* ClientConfig = GetDefault<UConcertClientConfig>();
	const bool bIsReadOnly = OnEvaluateRole.Execute(ClientConfig->ReadOnlyAssignment);
	const bool bIsSendOnly = OnEvaluateRole.Execute(ClientConfig->SendOnlyAssignment);


	if (bIsSendOnly && !bIsReadOnly)
	{
		return EConcertSendReceiveState::SendOnly;
	}
	if (bIsReadOnly && !bIsSendOnly)
	{
		return EConcertSendReceiveState::ReceiveOnly;
	}
	return EConcertSendReceiveState::Default;
}

EConcertSendReceiveState FConcertClientSession::GetSendReceiveState() const
{
	return SendReceiveState;
}

void LogSendReceiveStateChange(
	FConcertSessionInfo SessionInfo,
	EConcertSendReceiveState InState)
{
	FString State = [InState]()
	{
		if (InState == EConcertSendReceiveState::Default)
		{
			return FString("Resumed");
		}
		else if (InState == EConcertSendReceiveState::SendOnly)
		{
			return FString("SendOnly");
		}
		return FString("ReceiveOnly");
	}();
	UE_LOGF(LogConcert, Display,
		   "%ls Concert session '%ls' (Id: %ls, Owner: %ls).",
		   *State,
		   *SessionInfo.SessionName,
		   *SessionInfo.SessionId.ToString(),
		   *SessionInfo.OwnerUserName);
}

void FConcertClientSession::SetSendReceiveState(EConcertSendReceiveState InSendReceiveState)
{
	SendReceiveState = InSendReceiveState;
	LogSendReceiveStateChange(SessionInfo, SendReceiveState);
}

FOnConcertClientSessionTick& FConcertClientSession::OnTick()
{
	return OnTickDelegate;
}

FOnConcertClientSessionConnectionChanged& FConcertClientSession::OnConnectionChanged()
{
	return OnConnectionChangedDelegate;
}

FOnConcertClientSessionClientChanged& FConcertClientSession::OnSessionClientChanged()
{
	return OnSessionClientChangedDelegate;
}

FOnConcertSessionRenamed& FConcertClientSession::OnSessionRenamed()
{
	return OnSessionRenamedDelegate;
}

FString FConcertClientSession::GetSessionWorkingDirectory() const
{
	return SessionDirectory;
}

void FConcertClientSession::SendNextCustomEvent(const FConcertSequencedCustomEventManager::FPendingCustomEvent& PendingEvent)
{
	ClientSessionEndpoint->SendEvent(PendingEvent.CustomEvent, PendingEvent.ServerEndpointId, PendingEvent.Flags, PendingEvent.Annotations);
}

void FConcertClientSession::InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequenceId)
{
	if (DestinationEndpointIds.Num() == 0)
	{
		if (InSequenceId)
		{
			CustomEventSequenceManager.RemoveSequencedEvent(*InSequenceId);
		}
		return;
	}

	if (ConnectionStatus == EConcertConnectionStatus::Disconnected)
	{
		return;
	}

	// Build the event
	FConcertSession_CustomEvent CustomEvent;
	CommonBuildCustomEvent(EventType, EventData, GetSessionClientEndpointId(), DestinationEndpointIds, CustomEvent);

	// If the message is sent with the UniqueId flag, add an annotations so that we can uniquely identify multiple message bus copies of that message.
	TMap<FName, FString> Annotations;
	if (EnumHasAnyFlags(Flags, EConcertMessageFlags::UniqueId))
	{
		Annotations.Add(ConcertClientMessageIdName, FGuid::NewGuid().ToString());
	}

	using FPendingEvent = FConcertSequencedCustomEventManager::FPendingCustomEvent;
	FPendingEvent InEvent{
		MoveTemp(CustomEvent),
		Flags,
		MoveTemp(Annotations),
		SessionInfo.ServerEndpointId
	};

	const bool bNotReliablyOrdered = !EnumHasAnyFlags(Flags, EConcertMessageFlags::ReliableOrdered);
	const bool bHasCustomEventsPending = CustomEventSequenceManager.HasEventsPending();

	if (!bHasCustomEventsPending || bNotReliablyOrdered)
	{
		// We are not currently sequencing the events so send it immediately.
		SendNextCustomEvent(InEvent);
		return;
	}
	// If the user provided a id then use that id to store the value in the manager; otherwise create a new entry.
	FConcertSequencedCustomEvent EventId = InSequenceId ? *InSequenceId : CustomEventSequenceManager.AddSequencedCustomEvent();
	CustomEventSequenceManager.FillPendingSequenceEvent(EventId, MoveTemp(InEvent));

	while (TOptional<FPendingEvent> NextToSend = CustomEventSequenceManager.PopSendEvent())
	{
		SendNextCustomEvent(*NextToSend);
	}
}

void FConcertClientSession::InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler)
{
	// TODO: don't send if not connected

	// Build the request
	FConcertSession_CustomRequest CustomRequest;
	CommonBuildCustomRequest(RequestType, RequestData, GetSessionClientEndpointId(), DestinationEndpointId, CustomRequest);

	// Send the request
	ClientSessionEndpoint->SendRequest<FConcertSession_CustomRequest, FConcertSession_CustomResponse>(CustomRequest, SessionInfo.ServerEndpointId)
		.Next([Handler](const FConcertSession_CustomResponse& Response)
		{
			// Handle the response
			CommonHandleCustomResponse(Response, Handler);
		});
}

void FConcertClientSession::UpdateLocalClientInfo(const FConcertClientInfoUpdate& UpdatedFields)
{
	bool bUpdated = UpdatedFields.ApplyTo(ClientInfo);
	if (bUpdated)
	{
		// Notifies remote clients about the change.
		FConcertSession_UpdateClientInfoEvent ClientInfoUpdateEvent;
		ClientInfoUpdateEvent.SessionClient = FConcertSessionClientInfo{GetSessionClientEndpointId(), ClientInfo};
		ClientSessionEndpoint->SendEvent(ClientInfoUpdateEvent, SessionInfo.ServerEndpointId);
	}
}

void FConcertClientSession::HandleClientInfoUpdatedEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_UpdateClientInfoEvent* Message = Context.GetMessage<FConcertSession_UpdateClientInfoEvent>();
	if (FSessionClient* SessionClient = SessionClients.Find(Message->SessionClient.ClientEndpointId))
	{
		SessionClient->ClientInfo.ClientInfo = Message->SessionClient.ClientInfo;
		OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Updated, SessionClient->ClientInfo);
	}
}

void FConcertClientSession::HandleRemoteConnectionChanged(const FConcertEndpointContext& RemoteEndpointContext, EConcertRemoteEndpointConnection Connection)
{
	if (RemoteEndpointContext.EndpointId == SessionInfo.ServerEndpointId && (Connection == EConcertRemoteEndpointConnection::TimedOut || Connection == EConcertRemoteEndpointConnection::ClosedRemotely))
	{
		Disconnect();
	}
}

void FConcertClientSession::HandleJoinSessionResultEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_JoinSessionResultEvent* Message = Context.GetMessage<FConcertSession_JoinSessionResultEvent>();

	// Discard answer not from the expecting session
	if (Message->SessionServerEndpointId != SessionInfo.ServerEndpointId)
	{
		return;
	}

	// If we aren't actively connecting, discard the message
	if (ConnectionStatus != EConcertConnectionStatus::Connecting)
	{
		return;
	}

	// This should not trigger, it would mean that the server discovery mechanism, allowed a mismatched protocol version
	check(Message->ConcertProtocolVersion == EConcertMessageVersion::LatestVersion);

	// Check the session answer
	switch (Message->ConnectionResult)
	{
		// Connection was refused, go back to disconnected
		case EConcertConnectionResult::ConnectionRefused:
			ConnectionStatus = EConcertConnectionStatus::Disconnected;
			OnConnectionChangedDelegate.Broadcast(*this, ConnectionStatus);
			UE_LOGF(LogConcert, Display, "Disconnected from Concert session '%ls' (Id: %ls, Owner: %ls): Connection Refused.", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
			break;
		case EConcertConnectionResult::AlreadyConnected:
			// falls through
		case EConcertConnectionResult::ConnectionAccepted:
			ConnectionAccepted(Message->SessionClients);
			break;
		default:
			break;
	}
}

void FConcertClientSession::HandleClientListUpdatedEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_ClientListUpdatedEvent* Message = Context.GetMessage<FConcertSession_ClientListUpdatedEvent>();

	check(Message->ConcertEndpointId == SessionInfo.ServerEndpointId);

	UpdateSessionClients(Message->SessionClients);
}

void FConcertClientSession::HandleSessionRenamedEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_SessionRenamedEvent* Message = Context.GetMessage<FConcertSession_SessionRenamedEvent>();
	check(Message->ConcertEndpointId == SessionInfo.ServerEndpointId);

	FString OldName = SessionInfo.SessionName;
	SessionInfo.SessionName = Message->NewName;
	OnSessionRenamedDelegate.Broadcast(OldName, Message->NewName);
}

void FConcertClientSession::TickConnection(float DeltaSeconds, const FDateTime& UtcNow)
{
	if (LastConnectionTick + SessionTickFrequency <= UtcNow)
	{
		switch (ConnectionStatus)
		{
		case EConcertConnectionStatus::Connecting:
			SendConnectionRequest();
			break;
		default:
			// do nothing
			break;
		}
		LastConnectionTick = UtcNow;
	}

	// External callback when connected
	if (ConnectionStatus == EConcertConnectionStatus::Connected)
	{
		OnTickDelegate.Broadcast(*this, DeltaSeconds);
	}
}

void FConcertClientSession::SendConnectionRequest()
{
	FConcertSession_DiscoverAndJoinSessionEvent DiscoverAndJoinSessionEvent;
	DiscoverAndJoinSessionEvent.ConcertProtocolVersion = EConcertMessageVersion::LatestVersion;
	DiscoverAndJoinSessionEvent.SessionServerEndpointId = SessionInfo.ServerEndpointId;
	DiscoverAndJoinSessionEvent.ClientInfo = ClientInfo;
	ClientSessionEndpoint->PublishEvent(DiscoverAndJoinSessionEvent);
}

void FConcertClientSession::SendDisconnection()
{
	FConcertSession_LeaveSessionEvent LeaveSessionEvent;
	LeaveSessionEvent.SessionServerEndpointId = SessionInfo.ServerEndpointId;
	ClientSessionEndpoint->SendEvent(LeaveSessionEvent, SessionInfo.ServerEndpointId);
}

void FConcertClientSession::ConnectionAccepted(const TArray<FConcertSessionClientInfo>& InSessionClients)
{
	check(ConnectionStatus != EConcertConnectionStatus::Connected);
	ConnectionStatus = EConcertConnectionStatus::Connected;

	// Raise connected event
	OnConnectionChangedDelegate.Broadcast(*this, ConnectionStatus);

	UE_LOGF(LogConcert, Display, "Connected to Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);

	UpdateSessionClients(InSessionClients);
}

void FConcertClientSession::UpdateSessionClients(const TArray<FConcertSessionClientInfo>& InSessionClients)
{
	// Add any new clients, or update existing ones
	TSet<FGuid> AvailableClientIds;
	AvailableClientIds.Reserve(InSessionClients.Num());
	for (const FConcertSessionClientInfo& SessionClientInfo : InSessionClients)
	{
		if (ClientSessionEndpoint->GetEndpointContext().EndpointId != SessionClientInfo.ClientEndpointId)
		{
			AvailableClientIds.Add(SessionClientInfo.ClientEndpointId);

			if (SessionClients.Contains(SessionClientInfo.ClientEndpointId))
			{
				// TODO: Client updates?
				//OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Updated, SessionClientInfo);
			}
			else
			{
				const FSessionClient& SessionClient = SessionClients.Add(SessionClientInfo.ClientEndpointId, FSessionClient{ SessionClientInfo, MakeShared<FConcertScratchpad, ESPMode::ThreadSafe>() });
				OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Connected, SessionClientInfo);
				UE_LOGF(LogConcert, Display, "User '%ls' (Endpoint: %ls) joined Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionClient.ClientInfo.ClientInfo.UserName, *SessionClient.ClientInfo.ClientEndpointId.ToString(), *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
			}
		}
	}

	// Remove any old clients
	for (auto SessionClientIt = SessionClients.CreateIterator(); SessionClientIt; ++SessionClientIt)
	{
		if (!AvailableClientIds.Contains(SessionClientIt.Key()))
		{
			// Update array before broadcasting to ensure a client calling GetSessionClients() during broacast gets the up-to-date list.
			FSessionClient SessionClient = MoveTemp(SessionClientIt.Value());
			SessionClientIt.RemoveCurrent();

			OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Disconnected, SessionClient.ClientInfo);
			UE_LOGF(LogConcert, Display, "User '%ls' (Endpoint: %ls) left Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionClient.ClientInfo.ClientInfo.UserName, *SessionClient.ClientInfo.ClientEndpointId.ToString(), *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
		}
	}
}
