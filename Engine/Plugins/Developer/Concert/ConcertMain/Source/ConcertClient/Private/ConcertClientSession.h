// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/Guid.h"
#include "IConcertSession.h"
#include "ConcertSession.h"
#include "ConcertMessages.h"
#include "Containers/Ticker.h"
#include "GameplayTagContainer.h"

class IConcertLocalEndpoint;
struct FConcertClientSettings;
class FConcertClientSession;

/** Implementation of a Concert Client session */
class FConcertClientSession : public IConcertClientSession, private FConcertSessionCommonImpl
{
public:
	FConcertClientSession(const FConcertSessionInfo& InSessionInfo, const FConcertClientInfo& InClientInfo, const FConcertClientSettings& InSettings, TSharedPtr<IConcertLocalEndpoint> InClientSessionEndpoint, const FString& InSessionDirectory);
	virtual ~FConcertClientSession();

	virtual void Startup() override;
	virtual void Shutdown() override;

	virtual const FGuid& GetId() const override
	{
		return CommonGetId();
	}

	virtual const FString& GetName() const override
	{
		return CommonGetName();
	}

	virtual const FConcertSessionInfo& GetSessionInfo() const override
	{
		return CommonGetSessionInfo();
	}

	virtual TArray<FGuid> GetSessionClientEndpointIds() const override
	{
		return CommonGetSessionClientEndpointIds();
	}

	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override
	{
		return CommonGetSessionClients();
	}

	virtual bool FindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const override
	{
		return CommonFindSessionClient(EndpointId, OutSessionClientInfo);
	}

	virtual FConcertScratchpadRef GetScratchpad() const override
	{
		return CommonGetScratchpad();
	}

	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid& ClientEndpointId) const override
	{
		return CommonGetClientScratchpad(ClientEndpointId);
	}

	virtual EConcertConnectionStatus GetConnectionStatus() const override
	{
		return ConnectionStatus;
	}

	virtual FGuid GetSessionClientEndpointId() const override
	{
		return ClientSessionEndpoint->GetEndpointContext().EndpointId;
	}

	virtual FGuid GetSessionServerEndpointId() const override
	{
		return SessionInfo.ServerEndpointId;
	}

	virtual const FConcertClientInfo& GetLocalClientInfo() const override
	{
		return ClientInfo;
	}

	virtual void UpdateLocalClientInfo(const FConcertClientInfoUpdate& UpdatedFields) override;

	virtual void Connect() override;
	virtual void Disconnect() override;

	virtual EConcertSendReceiveState GetSendReceiveState() const override;
	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) override;

	virtual FOnConcertClientSessionTick& OnTick() override;
	virtual FOnConcertClientSessionConnectionChanged& OnConnectionChanged() override;
	virtual FOnConcertClientSessionClientChanged& OnSessionClientChanged() override;
	virtual FOnConcertSessionRenamed& OnSessionRenamed() override;
	virtual FString GetSessionWorkingDirectory() const override;

	virtual FConcertSequencedCustomEventManager& GetSequencedEventManager() override
	{
		return CustomEventSequenceManager;
	}

protected:
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) override
	{
		return CommonRegisterCustomEventHandler(EventMessageType, Handler);
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override
	{
		CommonUnregisterCustomEventHandler(EventMessageType, EventHandle);
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override
	{
		CommonUnregisterCustomEventHandler(EventMessageType, EventHandler);
	}

	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override
	{
		CommonClearCustomEventHandler(EventMessageType);
	}

	virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequencedId = {}) override;

	virtual void InternalRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler) override
	{
		CommonRegisterCustomRequestHandler(RequestMessageType, Handler);
	}

	virtual void InternalUnregisterCustomRequestHandler(const FName& RequestMessageType) override
	{
		CommonUnregisterCustomRequestHandler(RequestMessageType);
	}

	virtual void InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler) override;

private:
	/** */
	void SendNextCustomEvent(const FConcertSequencedCustomEventManager::FPendingCustomEvent& PendingEvent);

	/** */
	void HandleRemoteConnectionChanged(const FConcertEndpointContext& RemoteEndpointContext, EConcertRemoteEndpointConnection Connection);

	/**  */
	void HandleJoinSessionResultEvent(const FConcertMessageContext& Context);

	/**  */
	void HandleClientListUpdatedEvent(const FConcertMessageContext& Context);

	/** */
	void HandleSessionRenamedEvent(const FConcertMessageContext& Context);

	/**  */
	void HandleClientInfoUpdatedEvent(const FConcertMessageContext& Context);

	/**  */
	void TickConnection(float DeltaSeconds, const FDateTime& UtcNow);

	/**  */
	void SendConnectionRequest();

	/**  */
	void SendDisconnection();

	/**  */
	void ConnectionAccepted(const TArray<FConcertSessionClientInfo>& InSessionClients);

	/** */
	void UpdateSessionClients(const TArray<FConcertSessionClientInfo>& InSessionClients);

	/** Gets the default send receive state from the project settings. */
	EConcertSendReceiveState GetDefaultSendReceiveState();

	/** Information about this Client */
	FConcertClientInfo ClientInfo;

	/** The connection status to the server counterpart */
	EConcertConnectionStatus ConnectionStatus;

	/** This session endpoint where message are sent and received from. */
	IConcertLocalEndpointPtr ClientSessionEndpoint;

	/** State of the connection */
	EConcertSendReceiveState SendReceiveState = EConcertSendReceiveState::Default;

	/** Ticker for the session */
	FTSTicker::FDelegateHandle SessionTick;

	/** Last connection tick */
	FDateTime LastConnectionTick;

	/** Callback for when a connected session ticks */
	FOnConcertClientSessionTick OnTickDelegate;

	/** Callback for when the session connection state changes */
	FOnConcertClientSessionConnectionChanged OnConnectionChangedDelegate;

	/** Callback for when a session client state changes */
	FOnConcertClientSessionClientChanged OnSessionClientChangedDelegate;

	/** Callback when the session name changes. */
	FOnConcertSessionRenamed OnSessionRenamedDelegate;

	FConcertSequencedCustomEventManager CustomEventSequenceManager;

	/** The timespan at which session updates are processed. */
	const FTimespan SessionTickFrequency;

	/** The directory where this session will store its files */
	const FString SessionDirectory;
};
