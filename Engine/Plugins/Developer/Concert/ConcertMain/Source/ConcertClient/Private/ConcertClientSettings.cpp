// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertClientSettings)

UConcertClientConfig::UConcertClientConfig()
{
}
