// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "ConcertTransportSettings.h"
#include "ConcertVersion.h"
#include "GameplayTagContainer.h"

#include "ConcertClientSettings.generated.h"

#define UE_API CONCERTCLIENT_API

USTRUCT()
struct FConcertClientSettings
{
	GENERATED_BODY()

	FConcertClientSettings()
		: DisplayName()
		, AvatarColor(1.0f, 1.0f, 1.0f, 1.0f)
		, DesktopAvatarActorClass(TEXT("/ConcertSyncClient/DesktopPresence.DesktopPresence_C"))
		, VRAvatarActorClass(TEXT("/ConcertSyncClient/VRPresence.VRPresence_C"))
		, ServerPort(0)
		, DiscoveryTimeoutSeconds(5)
		, SessionTickFrequencySeconds(1)
		, LatencyCompensationMs(0)
	{}

	/** 
	 * The display name to use when in a session. 
	 * Can be specified on the editor cmd with `-CONCERTDISPLAYNAME=`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	FString DisplayName;

	/** The color used for the presence avatar in a session. */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	FLinearColor AvatarColor;

	/** The desktop representation of this editor's user to other connected users */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Client Settings", meta = (MetaClass = "/Script/ConcertSyncClient.ConcertClientDesktopPresenceActor"))
	FSoftClassPath DesktopAvatarActorClass;

	/** The VR representation of this editor's user to other connected users */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Client Settings", meta = (MetaClass = "/Script/ConcertSyncClient.ConcertClientVRPresenceActor", DisplayName = "VR Avatar Actor Class"))
	FSoftClassPath VRAvatarActorClass;

	/** The port to use to reach the server with static endpoints when launched through the editor. This port will be used over the unicast endpoint port in the UDP Messagging settings if non 0 when transferring the editor settings to the launched server. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Client Settings")
	uint16 ServerPort;

	/** The timespan at which discovered Multi-User server are considered stale if they haven't answered back */
	UPROPERTY(config, EditAnywhere, DisplayName="Discovery Timeout", AdvancedDisplay, Category="Client Settings", meta=(ForceUnits=s))
	int32 DiscoveryTimeoutSeconds;

	/** The timespan at which session updates are processed. */
	UPROPERTY(config, EditAnywhere, DisplayName="Session Tick Frequency", AdvancedDisplay, Category="Client Settings", meta=(ForceUnits=s))
	int32 SessionTickFrequencySeconds;

	/** Amount of latency compensation to apply to time-synchronization sensitive interactions */
	UPROPERTY(config, EditAnywhere, DisplayName="Latency Compensation", AdvancedDisplay, Category="Client Settings", meta=(ForceUnits=ms))
	float LatencyCompensationMs;

	UPROPERTY()
	bool bReflectLevelEditorInGame_DEPRECATED = false;

	/**
	 * Enable extended version support when using Multi-user with precompiled and source builds.  When using Unreal Game
	 * Sync, it is possible to have the same engine CL but different engine version due to content changes.	 This setting
	 * enables reading engine version CL from the Build.version file produced by UGS to determine engine version
	 * information when joining a session.	This only applies when you intend to mix precompiled with source builds.
	*/
	UPROPERTY(config, EditAnywhere, DisplayName="Support Mixed Build Types", AdvancedDisplay, Category="Client Settings")
	bool bSupportMixedBuildTypes = false;

	/** Array of tags that can be used for grouping and categorizing. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Client Settings")
	TArray<FName> Tags;

	/** A key used to identify the clients during server discovery. If the server was configured to restrict access, the client key must be know of the server. Can be left empty. */
	UPROPERTY(config)
	FString ClientAuthenticationKey;

	/**
	 * Specifies if we should use a file sharing service for larger assets that are saved in a Multi-user session. Note the specified folder must be
	 * visible to all clients.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "File Sharing")
	bool bEnableFileSharing = false;

	/**
	 * Indicates that the file sharing content should be removed on exit. If this is false then it will be left in place and must be removed by the user.
	 */
	UPROPERTY(Config, EditAnywhere, meta=(EditCondition="bEnableFileSharing"), Category = "File Sharing")
	bool bCleanSessionFilesOnDisconnect = true;

	/**
	 * The path for that shares files across clients. This path must be visible to all nodes. A challenge request will be sent to the server to confirm that it can see the path.
	 * If the server cannot see the path then it will reject file sharing for this node and it will fallback to a non-fileshare code flow. 
	 */
	UPROPERTY(Config, EditAnywhere, meta=(EditCondition="bEnableFileSharing"), Category = "File Sharing")
	FDirectoryPath CloudFileSharingPath;
};

UENUM()
enum class EConcertSourceValidationMode : uint8
{
	/** Source control validation will fail on any changes when connecting to a Multi-User Session. */
	Hard = 0,
	/** 
	 * Source control validation will warn and prompt on any changes when connecting to a Multi-User session. 
	 * In Memory changes will be hot-reloaded.
	 * Source control changes aren't affected but will be stashed/shelved in the future.
	 */
	Soft,
	/** Soft validation mode with auto proceed on prompts. */
	SoftAutoProceed
};

USTRUCT()
struct FConcertSourceControlSettings
{
	GENERATED_BODY()

	FConcertSourceControlSettings()
		: ValidationMode(EConcertSourceValidationMode::Soft)
	{}

	UPROPERTY(config, EditAnywhere, Category="Revision Control Settings", meta=(Keywords = "Source Control"))
	EConcertSourceValidationMode ValidationMode;
};

UENUM()
enum class EConcertServerType
{
	Console,
	Slate
};

UCLASS(MinimalAPI, config=Engine)
class UConcertClientConfig : public UObject
{
	GENERATED_BODY()
public:
	UE_API UConcertClientConfig();

	/*
	 * Mark this setting object as editor only.
	 * This so soft object path reference made by this setting object won't be automatically grabbed by the cooker.
	 * @see UPackage::Save, FSoftObjectPathThreadContext::GetSerializationOptions, FSoftObjectPath::ImportTextItem
	 * @todo: cooker should have a better way to filter editor only objects for 'unsolicited' references.
	 */
	virtual bool IsEditorOnly() const
	{
		return true;
	}

	/**
	 * True if this client should be "headless"? (ie, not display any UI).
	 */
	UPROPERTY(config)
	bool bIsHeadless;

	/**
	 * True if the Multi-User module should install shortcut button and its drop-down menu in the level editor toolbar.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings", Meta=(ConfigRestartRequired=true, DisplayName="Enable Multi-User Toolbar Button"))
	bool bInstallEditorToolbarButton;

	/** 
	 * Automatically connect or create default session on default server.
	 * Can be specified on the editor cmd with `-CONCERTAUTOCONNECT` or `-CONCERTAUTOCONNECT=<true/false>`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	bool bAutoConnect;

	/** 
	 * If auto-connect is on, retry connecting to the default server/session until it succeeds or the user cancels.
	 * Can be specified on the editor cmd with `-CONCERTRETRYAUTOCONNECTONERROR` or `-CONCERTRETRYAUTOCONNECTONERROR=<true/false>`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	bool bRetryAutoConnectOnError = false;

	/**
	 * Determines which server executable with be launched:
	 *		Console -> UnrealMultiUserServer.exe
	 *		Slate	-> UnrealMultiUserSlateServer.exe
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	EConcertServerType ServerType = EConcertServerType::Slate;

	/** 
	 * Default server url (just a name for now) to look for on auto or default connect. 
 	 * Can be specified on the editor cmd with `-CONCERTSERVER=`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	FString DefaultServerURL;

	/** 
	 * Default session name to look for on auto connect or default connect.
	 * Can be specified on the editor cmd with `-CONCERTSESSION=`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	FString DefaultSessionName;

	/**
	 * If this client create the default session, should the session restore a saved session.
	 * Set the name of the desired save to restore its content in your session.
	 * Leave this blank if you want to create an empty session.
	 * Can be specified on the editor cmd with `-CONCERTSESSIONTORESTORE=`.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Client Settings")
	FString DefaultSessionToRestore;

	/**
	 * If this client create the default session, should the session data be saved when it's deleted.
	 * Set the name desired for the save and the session data will be moved in that save when the session is deleted
	 * Leave this blank if you don't want to save the session data.
	 * Can be specified on the editor cmd with `-CONCERTSAVESESSIONAS=`.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Client Settings")
	FString DefaultSaveSessionAs;

	/**
	 * A list of roles that should enter multi-user in read-only mode. If a role is in both SendOnlyAssignment and
	 * ReadOnlyassignment then the client will enter the session as full read/write.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Client Settings")
	FGameplayTagContainer ReadOnlyAssignment;

	/**
	 * A list of roles that should enter multi-user in send-only mode. If a role is in both SendOnlyAssignment and
	 * ReadOnlyAssignment then the client will enter the session as full read/write.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Client Settings")
	FGameplayTagContainer SendOnlyAssignment;

	/**
	 * Hot reload of a level happens automatically when level / sublevel is changed. This property allows users
	 * to receive a prompt before hot reload occurs so that it does not interrupt the user in the current level.
	 * Can be specified on the editor command with `-CONCERTSHOULDPROMPTFORHOTRELOAD`
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	bool bShouldPromptForHotReloadOnLevel = false;

	/** Client & client session settings */
	UPROPERTY(config, EditAnywhere, Category="Client Settings", meta=(ShowOnlyInnerProperties))
	FConcertClientSettings ClientSettings;

	UPROPERTY(config, EditAnywhere, Category = "Revision Control Settings", meta=(ShowOnlyInnerProperties, Keywords = "Source Control"))
	FConcertSourceControlSettings SourceControlSettings;

	/** Endpoint settings passed down to endpoints on creation */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Endpoint Settings", meta=(ShowOnlyInnerProperties))
	FConcertEndpointSettings EndpointSettings;
};

#undef UE_API
