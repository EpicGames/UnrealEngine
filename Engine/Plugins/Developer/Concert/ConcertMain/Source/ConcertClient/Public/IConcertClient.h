// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertSession.h"
#include "CoreMinimal.h"
#include "Async/Future.h"
#include "IConcertSession.h"
#include "ConcertMessages.h"
#include "ConcertTransportMessages.h"
#include "IConcertFileSharingService.h"

class UConcertClientConfig;

class IConcertClient;
class IConcertClientConnectionTask;
struct FGameplayTagContainer;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertClientSessionStartupOrShutdown, TSharedRef<IConcertClientSession>);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientSessionGetPreConnectionTasks, const IConcertClient&, TArray<TUniquePtr<IConcertClientConnectionTask>>&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnConcertEvaluateHasRole, const FGameplayTagContainer&);

namespace UE::ConcertClient
{
CONCERTCLIENT_API FOnConcertEvaluateHasRole& VPRoleEvaluator();
}
/**
 * Enum to indicate if any action were taken for the connection task
 */
enum class EConcertConnectionTaskAction : uint8
{
	None = 0,
	Cancel,
	Continue
};

/**
 * Struct to contain connection error
 * Code is an integer instead of an enum since connection task can be extended, range under 10000 is reserved.
 * Currently used code and their meaning are as follow
 *	No Error								= 0:	Success
 *	Pre Connection Canceled					= 1:	Pre connection validation was canceled by the user
 *	Connection Attempt Aborted				= 2:	Ongoing connection attempt was canceled by the user
 *	Server Not Responding					= 3:	Connection request to server timed out
 *	Server Request Failure					= 4:	Server refused the session connection request
 *	Workspace Validation Unknown Error		= 100:	Workspace validation unknown error
 *	Source Control Validation Unknown Error	= 110:	Unknown source control validation error
 *	Source Control Validation Canceled		= 111:	Source control validation was canceled by the user
 *	Source Control Validation Error			= 112:	Modified files not yet submitted were found in the workspace
 *	Dirty Packages Validation Error			= 113:	Dirty Packages were found before connecting to a session
 */
struct FConcertConnectionError
{
	/** Code for the last connection error. */
	uint32 ErrorCode = 0;
	/** Localized text associated with the error. */
	FText ErrorText;
};

/** Interface for tasks executed during the Concert client connection flow (eg, validation, creation, connection) */
class IConcertClientConnectionTask
{
public:
	virtual ~IConcertClientConnectionTask() = default;

	/**
	 * Execute this task.
	 * Typically this puts the task into a pending state, however it is possible for the task to immediately complete once executed. Ideally this should not block for a long time!
	 */
	virtual void Execute() = 0;

	/**
	 * Abort this task immediately, and discard any pending work.
	 * @note It is expected that GetStatus and GetError will return some kind of error state after this has been called.
	 */
	virtual void Abort() = 0;

	/**
	 * Tick this task, optionally requesting that it should gracefully cancel.
	 */
	virtual void Tick(EConcertConnectionTaskAction TaskAction) = 0;

	/**
	 * Get whether this task can be gracefully canceled.
	 */
	virtual bool CanCancel() const = 0;

	/**
	 * Get the current status of this task.
	 * @note It is required that the task return Pending while it is in-progress, and Success when it has finished successfully. Any other status is treated as an error state, and GetError will be called.
	 */
	virtual EConcertResponseCode GetStatus() const = 0;

	/**
	 * Get a description of this task that can be used in the progress notification (if any).
	 */
	virtual FText GetDescription() const = 0;

	/**
	 * Get the prompt message of this task to be displayed on prompt button if the task require action.
	 */
	virtual FText GetPrompt() const = 0;

	/**
	 * Get the extended error status of this task that can be used in the error notification (if any).
	 */
	virtual FConcertConnectionError GetError() const = 0;

	/**
	 * Get the delegate to gather more error details for this task error notification (if any).
	 */
	virtual FSimpleDelegate GetErrorDelegate() const = 0;
};

struct FConcertCreateSessionArgs
{
	/** The desired name for the session */
	FString SessionName;

	/** The override for the name used when archiving this session */
	FString ArchiveNameOverride;
};

struct FConcertCopySessionArgs
{
	/** True to auto-connect to the session after copying/restoring it */
	bool bAutoConnect = true;

	/** The ID of the session to copy or restore */
	FGuid SessionId;

	/** The desired name for new session */
	FString SessionName;

	/** The override for the name used when archiving the copied/restored session */
	FString ArchiveNameOverride;

	/** The filter controlling which activities should be copied from the source session. */
	FConcertSessionFilter SessionFilter;
};

struct FConcertArchiveSessionArgs
{
	/** The ID of the archived session to archive */
	FGuid SessionId;

	/** The override for the name used when archiving the session */
	FString ArchiveNameOverride;

	/** The filter controlling which activities from the session should be archived */
	FConcertSessionFilter SessionFilter;
};

struct FConcertBatchDeleteSessionsArgs
{
	TSet<FGuid> SessionIds;
	EBatchSessionDeletionFlags Flags;
};

/** Interface for Concert client */
class IConcertClient
{
public:
	virtual ~IConcertClient() = default;

	/**
	 * Get the role of this client (eg, MultiUser, DisasterRecovery, etc)
	 */
	virtual const FString& GetRole() const = 0;

	/** 
	 * Configure the client settings and its information.
	 * @note If Configure() is called while the client is in a session, some settings may be applied only once the client leave the session.
	 */
	virtual void Configure(const UConcertClientConfig* InSettings) = 0;

	/**
	 * Return true if the client has been configured.
	 */
	virtual bool IsConfigured() const = 0;

	/**
	 * Return The configuration of this client, or null if it hasn't been configured.
	 */
	virtual const UConcertClientConfig* GetConfiguration() const = 0;

	/**
	 * Get the client information passed to Configure() if the client is not in a session, otherwise, returns
	 * the current session client info as returned by IConcertClientSession::GetLocalClientInfo().
	 */
	virtual const FConcertClientInfo& GetClientInfo() const = 0;

	/**
	 * Returns if the client has already been started up.
	 */
	virtual bool IsStarted() const = 0;

	/**
	 * Startup the client, this can be called multiple time
	 * Configure needs to be called before startup
	 */
	virtual void Startup() = 0;

	/**
	 * Shutdown the client, its discovery and session, if any.
	 * This can be called multiple time with no ill effect.
	 * However it depends on the UObject system so need to be called before its exit.
	 */
	virtual void Shutdown() = 0;
	
	/**
	 * Returns true if server discovery is enabled.
	 */
	virtual bool IsDiscoveryEnabled() const = 0;

	/**
	 * Start the discovery service for the client
	 * This will look for Concert server and populate the known servers list
	 * @see GetKnownServers
	 */
	virtual void StartDiscovery() = 0;

	/**
	 * Stop the discovery service for the client
	 */
	virtual void StopDiscovery() = 0;

	/**
	 * Returns true if the client is configured for auto connection.
	 */
	virtual bool CanAutoConnect() const = 0;

	/**
	 * Returns true if the client has an active auto connection routine.
	 */
	virtual bool IsAutoConnecting() const = 0;

	/**
	 * Start attempting to auto connect the client to the default session on the default server.
	 */
	virtual void StartAutoConnect() = 0;

	/**
	 * Stop the current auto connection if currently enabled.
	 */
	virtual void StopAutoConnect() = 0;

	/**
	 * Get the last connection error
	 * @return an error code along with a text description of the last connection error.
	 */
	virtual FConcertConnectionError GetLastConnectionError() const = 0;

	/**
	 * Get the list of discovered server information
	 */
	virtual TArray<FConcertServerInfo> GetKnownServers() const = 0;

	/**
	 * Get the delegate callback for when the known server list is updated
	 */
	virtual FSimpleMulticastDelegate& OnKnownServersUpdated() = 0;

	/**
	 * Get the delegate that is called right before the client session startup
	 */
	virtual FOnConcertClientSessionStartupOrShutdown& OnSessionStartup() = 0;

	/**
	 * Get the delegate that is called right before the client session shutdown
	 */
	virtual FOnConcertClientSessionStartupOrShutdown& OnSessionShutdown() = 0;

	/**
	 * Get the delegate that is called to get the pre-connection tasks for a client session
	 */
	virtual FOnConcertClientSessionGetPreConnectionTasks& OnGetPreConnectionTasks() = 0;

	/**
	 * Get the delegate that is called when the session connection state changes
	 */
	virtual FOnConcertClientSessionConnectionChanged& OnSessionConnectionChanged() = 0;

	/**
	 * Sets the handle to the file sharing service.  The client needs to prove to the server that they both can see the
	 * service.  This file sharing service will ultimate get promoted to the workspace to be used during a working session.
	 */
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) = 0;

	/**
	 * Get the current file sharing service held by the client. 
	 */
	virtual TSharedPtr<IConcertFileSharingService> GetFileSharingService() const = 0;		

	/**
	 * Get the connection status of client session or disconnected if no session is present
	 * @see EConcertConnectionStatus
	 */
	virtual EConcertConnectionStatus GetSessionConnectionStatus() const = 0;

	/** 
	 * Create a session on the server, matching the client configured settings.
	 * This also initiates the connection handshake for that session with the client.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param CreateSessionArgs		The arguments that will be use for the creation of the session
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> CreateSession(const FGuid& ServerAdminEndpointId, const FConcertCreateSessionArgs& CreateSessionArgs) = 0;

	/**
	 * Join a session on the server, the settings of the sessions needs to be compatible with the client settings
	 * or the connection will be refused.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param SessionId				The Id of the session
	 * @return  A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> JoinSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) = 0;

	/**
	 * Copy an archived session into a new live session on the server, matching the client configured settings.
	 * This also initiates the connection handshake for that session with the client when bAutoConnect is true in the RestoreSessionArgs.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param RestoreSessionArgs	The arguments that will be use for the restoration of the session
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> RestoreSession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& RestoreSessionArgs) = 0;

	/**
	 * Copy a live or and archived session into a new live session on server, matching the configured settings. If the session is archived,
	 * this is equivalent to restoring it (because copying an archive into another archive is not useful).
	 * This also initiates the connection handshake for that session with the server when bAutoConnect is true in the CopySessionArgs.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param CopySessionArgs	The arguments that will be use to copy the session
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> CopySession(const FGuid& ServerAdminEndpointId, const FConcertCopySessionArgs& CopySessionArgs) = 0;

	/**
	 * Archive a live session on the server hosting the session.
	 * @param ServerAdminEndpointId	The Id of the Concert Server hosting the session (and where the archive will be created)
	 * @param ArchiveSessionArgs	The arguments that will be use for the archiving of the session
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> ArchiveSession(const FGuid& ServerAdminEndpointId, const FConcertArchiveSessionArgs& ArchiveSessionArgs) = 0;

	/**
	 * Rename a live or archived session if the client has the permission. The server automatically detects if the session is live or archived.
	 * If the client is not the owner the request will be refused.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param SessionId				The Id of the live session to rename
	 * @param NewName				The new session name
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> RenameSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, const FString& NewName) = 0;

	/**
	 * Delete a live or archived session from the server if the client is the owner of the session. The server automatically detects if the session is live or archived.
	 * If the client is not the owner the request will be refused.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param SessionId				The Id of the session to delete
	 * @return A future that will contains the final response code of the request
	 */
	virtual TFuture<EConcertResponseCode> DeleteSession(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) = 0;

	/**
	 * Deletes several live or archives sessions from the server. If the client is not sure whether the client has permission for it,
	 * it can set the SkipForbiddenSessions - the server will skip disallowed sessions instead of rejecting the entire operation.
	 * @param ServerAdminEndpointId	The Id of the Concert Server query endpoint
	 * @param BatchDeletionArgs		The arguments that will be used for batch deleting the session(s)
	 * @return A future that will contain the server's response (and optionally which sessions were skipped due to permissions)
	 */
	virtual TFuture<FConcertAdmin_BatchDeleteSessionResponse> BatchDeleteSessions(const FGuid& ServerAdminEndpointId, const FConcertBatchDeleteSessionsArgs& BatchDeletionArgs) = 0;
	
	/** 
	 * Disconnect from the current session.
	 */
	virtual void DisconnectSession() = 0;

	/** Get the send/receive state for this session. */
	virtual EConcertSendReceiveState GetSendReceiveState() const = 0;

	/** Set the send/receive state for this session. */
	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) = 0;

	/**
	 * Does the client think it is the owner of the session?
	 */
	virtual bool IsOwnerOf(const FConcertSessionInfo& InSessionInfo) const = 0;

	/**
	 * Get the current client session (if any).
	 */
	virtual TSharedPtr<IConcertClientSession> GetCurrentSession() const = 0;

	/**
	 * Create or load a repository (directory structure) containing sessions. The server internally maps the ID to its directories.
	 * If the server finds the repository, it will scan the corresponding directories, discover the sessions, process them and
	 * make them visible to the other clients. If the repository is not found, but bCreateIfNotExist is true, a new repository will
	 * be created. Repositories are persistent and visible to all server instances until dropped by the clients. Any server instance
	 * can mount a repository unless it is already mounted by another instance.
	 * @note Usually, a client would create a new empty repository and set it as default to populate it, then keep the ID to reload/drop it later.
	 * @param ServerAdminEndpointId The server for which the workspace must be mounted.
	 * @param RepositoryRootDir The base directory containing the repository to create or load. Can hold several repositories. Leave it empty to use the server one if the client/server are not running on the same machine.
	 * @param RepositoryId A unique Id for the repository. Must be valid.
	 * @param bCreateIfNotExist Create a new repository if the specified one doesn't exist.
	 * @param bAsDefault Whether this repository should be set as the default one to store newly created sessions.
	 */
	virtual TFuture<FConcertAdmin_MountSessionRepositoryResponse> MountSessionRepository(const FGuid& ServerAdminEndpointId, const FString& RepositoryRootDir, const FGuid& RepositoryId, bool bCreateIfNotExist, bool bAsDefault = false) const = 0;

	/**
	 * Request the list of existing session repositories from the server.
	 */
	virtual TFuture<FConcertAdmin_GetSessionRepositoriesResponse> GetSessionRepositories(const FGuid& ServerAdminEndpointId) const = 0;

	/**
	 * Drop a set of repositories (and delete the files) from the server. All sessions stored in the unmounted repository will be unloaded and unreachable.
	 * The server can only drop the repository if it is not mounted or has mounted it itself.
	 */
	virtual TFuture<FConcertAdmin_DropSessionRepositoriesResponse> DropSessionRepositories(const FGuid& ServerAdminEndpointId, const TArray<FGuid>& RepositoryIds) const = 0;

	/**
	 * Get the list of sessions available on a server
	 * @param ServerAdminEndpointId The Id of the Concert server admin endpoint
	 * @return A future for FConcertAdmin_GetAllSessionsResponse which contains a list of sessions
	 */
	virtual TFuture<FConcertAdmin_GetAllSessionsResponse> GetServerSessions(const FGuid& ServerAdminEndpointId) const = 0;

	/**
	 * Get the list of the live sessions data from a server
	 * @param ServerAdminEndpointId	The Id of the concert sever admin endpoint
	 * @return A future for FConcertAdmin_GetSessionsResponse which contains the list of the archived sessions.
	 */
	virtual TFuture<FConcertAdmin_GetSessionsResponse> GetLiveSessions(const FGuid& ServerAdminEndpointId) const = 0;

	/**
	 * Get the list of the archived sessions data from a server
	 * @param ServerAdminEndpointId	The Id of the concert sever admin endpoint
	 * @return A future for FConcertAdmin_GetSessionsResponse which contains the list of the archived sessions.
	 */
	virtual TFuture<FConcertAdmin_GetSessionsResponse> GetArchivedSessions(const FGuid& ServerAdminEndpointId) const = 0;

	/**
	 * Get the list of clients connected to a session on the server
	 * @param ServerAdminEndpointId	The Id of the Concert server admin endpoint
	 * @param SessionId				The Id of the session
	 * @return A future for FConcertAdmin_GetSessionClientsResponse which contains a list of session clients
	 */
	virtual TFuture<FConcertAdmin_GetSessionClientsResponse> GetSessionClients(const FGuid& ServerAdminEndpointId, const FGuid& SessionId) const = 0;

	/**
	 * Get the specified session activities, ordered by Activity ID (ascending) from a live or archived session without being connected to it. The function is used
	 * to explore the history of a session, for example to implement the disaster recovery scenario. It is possible to get the total number of activities in a
	 * session using -1 as ActivityCount. The response will contain the last Activity and its ID. To get the N last activities, set ActivityCount = -N.
	 * @param ServerAdminEndpointId	The Id of the Concert server admin endpoint
	 * @param SessionId				The Id of the session
	 * @param FromActivityId		The first activity ID to fetch (1-based) if ActivityCount is positive. Ignored if ActivityCount is negative.
	 * @param ActivityCount			If positive, request \a ActivityCount starting from \a FromActivityId. If negative, request the Abs(\a ActivityCount) last activities.
	 * @param bIncludeDetails		If true, includes extra information for package and transaction activity types. See the return type for more info.
	 * @return A future for FConcertAdmin_GetSessionActivitiesResponse which contains up to Abs(ActivityCount) activities or an error if it fails. The array of payload in the response can always be
	 *         decoded as FConcertSyncActivity objects. If bIncludeDetails is true, 'package' and 'transaction' can be decoded respectively as FConcertSyncPackageActivity and FConcertSyncTransactionActivity,
	 *         providing extra information. The transaction event will contains the full transaction data. The package event contains the package event meta data only.
	 */
	virtual TFuture<FConcertAdmin_GetSessionActivitiesResponse> GetSessionActivities(const FGuid& ServerAdminEndpointId, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, bool bIncludeDetails) const = 0;
};
