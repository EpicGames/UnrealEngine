// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClient.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

struct FConcertSessionFilter;

typedef TSharedPtr<IConcertClient, ESPMode::ThreadSafe> IConcertClientPtr;
typedef TSharedRef<IConcertClient, ESPMode::ThreadSafe> IConcertClientRef;

/**
 * Interface for the Main Concert Server module.
 */
class IConcertClientModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IConcertClientModule& Get()
	{
		static const FName ModuleName = "ConcertClient";
		return FModuleManager::LoadModuleChecked<IConcertClientModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "ConcertClient";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Create a client that will perform a certain role (eg, MultiUser, DisasterRecovery, etc)
	 * @param InRole The role to create
	 * @return The client
	 */
	virtual IConcertClientRef CreateClient(const FString& InRole) = 0;
};
