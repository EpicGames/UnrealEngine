// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertServer : ModuleRules
	{
		public ConcertServer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",  // FH: Can we do without at some point?
					"ConcertTransport",
					"Serialization",
					"Concert"
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"ConcertTransport"
				}
			);
		}
	}
}
