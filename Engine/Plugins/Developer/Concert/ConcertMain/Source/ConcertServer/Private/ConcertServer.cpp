// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServer.h"

#include "ConcertUtil.h"
#include "ConcertServerUtil.h"
#include "ConcertServerSettings.h"
#include "ConcertServerSession.h"
#include "ConcertServerSessionRepositories.h"
#include "ConcertLogGlobal.h"
#include "ConcertTransportEvents.h"
#include "IConcertServerEventSink.h"
#include "IConcertFileSharingService.h"

#include "Algo/AnyOf.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "HAL/FileManager.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Templates/NonNullPointer.h"

#include "Runtime/Launch/Resources/Version.h"

#define LOCTEXT_NAMESPACE "ConcertServer"

namespace ConcertServerUtil
{

static const TCHAR* GetServerSystemMutexName()
{
	// A system wide mutex name used by this application instances that will unlikely be found in other applications.
	return TEXT("Unreal_ConcertServer_67822dAB");
}

static FString GetArchiveName(const FString& SessionName, const FConcertSessionSettings& Settings)
{
	if (Settings.ArchiveNameOverride.IsEmpty())
	{
		return FString::Printf(TEXT("%s_%s"), *SessionName, *FDateTime::UtcNow().ToString());
	}
	else
	{
		return Settings.ArchiveNameOverride;
	}
}

static FString GetSessionRepositoryDatabasePathname(const FString& Role)
{
	return FPaths::ProjectSavedDir() / Role / TEXT("Repositories.json");
}

static bool SaveSessionRepositoryDatabase(const FString& Role, const FConcertServerSessionRepositoryDatabase& RepositoryDb)
{
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*GetSessionRepositoryDatabasePathname(Role))))
	{
		FJsonStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(RepositoryDb, Backend);

		FileWriter->Close();
		return !FileWriter->IsError();
	}

	return false;
}

static bool LoadSessionRepositoryDatabase(const FString& Role, FConcertServerSessionRepositoryDatabase& RepositoryDb)
{
	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*GetSessionRepositoryDatabasePathname(Role))))
	{
		FJsonStructDeserializerBackend Backend(*FileReader);
		FStructDeserializer::Deserialize(RepositoryDb, Backend);

		FileReader->Close();
		return !FileReader->IsError();
	}

	return false;
}
}

FConcertServer::FConcertServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter, IConcertServerEventSink* InEventSink, const TSharedPtr<IConcertEndpointProvider>& InEndpointProvider)
	: Role(InRole)
	, DefaultSessionRepositoryStatus(LOCTEXT("SessionRepository_NotConfigured", "Repository not configured."))
	, AutoArchiveSessionFilter(InAutoArchiveSessionFilter)
	, EventSink(InEventSink)
	, EndpointProvider(InEndpointProvider)
{
	check(EventSink);
}

FConcertServer::~FConcertServer()
{
	// if ServerAdminEndpoint is valid, then Shutdown wasn't called
	check(!ServerAdminEndpoint.IsValid());
}

const FString& FConcertServer::GetRole() const
{
	return Role;
}

void FConcertServer::Configure(const UConcertServerConfig* InSettings)
{
	check(!Settings); // Server do not support reconfiguration.

	ServerInfo.Initialize();
	check(InSettings != nullptr);
	Settings = TStrongObjectPtr<const UConcertServerConfig>(InSettings);

	if (!InSettings->ServerName.IsEmpty())
	{
		ServerInfo.ServerName = InSettings->ServerName;
	}

	if (InSettings->ServerSettings.bIgnoreSessionSettingsRestriction)
	{
		ServerInfo.ServerFlags |= EConcertServerFlags::IgnoreSessionRequirement;
	}

	if (Settings->bEnableFileSharing && FileSharingService)
	{
		FileSharingService->SetEnabled(true);
		FileSharingService->SetPath(Settings->RootFileSharingPath.Path);
	}

	SessionRepositoryRootDir = FPaths::ProjectSavedDir() / Role / TEXT("Sessions"); // Server default session repository root dir.
	if (!Settings->SessionRepositoryRootDir.IsEmpty())
	{
		if (IFileManager::Get().DirectoryExists(*Settings->SessionRepositoryRootDir) || IFileManager::Get().MakeDirectory(*Settings->SessionRepositoryRootDir, /*Tree*/true))
		{
			SessionRepositoryRootDir = Settings->SessionRepositoryRootDir; // Overwrite the default.
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "Invalid session repository root directory. Falling back on %ls default.", *SessionRepositoryRootDir);
		}
	}
}

bool FConcertServer::IsConfigured() const
{
	// if the instance id hasn't been set yet, then Configure wasn't called.
	return Settings && ServerInfo.InstanceInfo.InstanceId.IsValid();
}

const UConcertServerConfig* FConcertServer::GetConfiguration() const
{
	return Settings.Get();
}

const FConcertServerInfo& FConcertServer::GetServerInfo() const
{
	return ServerInfo;
}

void FConcertServer::SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService)
{
	FileSharingService = MoveTemp(InFileSharingService);
	if (FileSharingService && Settings && Settings->bEnableFileSharing)
	{
		FileSharingService->SetEnabled(true);
		FileSharingService->SetPath(Settings->RootFileSharingPath.Path);
	}
}

TSharedPtr<IConcertFileSharingService> FConcertServer::GetFileSharingService() const
{
	return FileSharingService;
}

TArray<FConcertEndpointContext> FConcertServer::GetRemoteAdminEndpoints() const
{
	if (IsStarted())
	{
		return ServerAdminEndpoint->GetRemoteEndpoints();
	}
	return {};
}

FOnConcertRemoteEndpointConnectionChanged& FConcertServer::OnRemoteEndpointConnectionChanged()
{
	return OnConcertRemoteEndpointConnectionChangedDelegate;
}

FMessageAddress FConcertServer::GetRemoteAddress(const FGuid& AdminEndpointId) const
{
	if (IsStarted())
	{
		return ServerAdminEndpoint->GetRemoteAddress(AdminEndpointId);
	}
	return {};
}

FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& FConcertServer::OnConcertMessageAcknowledgementReceived()
{
	return OnConcertMessageAcknowledgementReceivedFromLocalEndpoint;
}

bool FConcertServer::IsStarted() const
{
	return ServerAdminEndpoint.IsValid();
}

void FConcertServer::Startup()
{
	check(IsConfigured());
	if (!ServerAdminEndpoint.IsValid() && EndpointProvider.IsValid())
	{
		// Create the server administration endpoint
		ServerAdminEndpoint = EndpointProvider->CreateLocalEndpoint(TEXT("Admin"), Settings->EndpointSettings, [this](const FConcertEndpointContext& Context)
		{
			return ConcertUtil::CreateLogger(Context, [this](const FConcertLog& Log)
			{
				ConcertTransportEvents::OnConcertServerLogEvent().Broadcast(*this, Log);
			});
		});
		ServerInfo.AdminEndpointId = ServerAdminEndpoint->GetEndpointContext().EndpointId;
		ServerAdminEndpoint->OnConcertMessageAcknowledgementReceived().AddLambda(
		[this](const FConcertEndpointContext& LocalEndpoint, const FConcertEndpointContext& RemoteEndpoint, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& MessageContext)
			{
				OnConcertMessageAcknowledgementReceivedFromLocalEndpoint.Broadcast(LocalEndpoint, RemoteEndpoint, AckedMessage, MessageContext);
			});
		ServerAdminEndpoint->OnRemoteEndpointConnectionChanged().AddLambda([this](const FConcertEndpointContext& Context, EConcertRemoteEndpointConnection Connection)
		{
			OnConcertRemoteEndpointConnectionChangedDelegate.Broadcast(Context, Connection);
		});

		// Make it discoverable
		ServerAdminEndpoint->SubscribeEventHandler<FConcertAdmin_DiscoverServersEvent>(this, &FConcertServer::HandleDiscoverServersEvent);
		
		// Add Session connection handling
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_CreateSessionRequest, FConcertAdmin_SessionInfoResponse>(this, &FConcertServer::HandleCreateSessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_FindSessionRequest, FConcertAdmin_SessionInfoResponse>(this, &FConcertServer::HandleFindSessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_CopySessionRequest, FConcertAdmin_SessionInfoResponse>(this, &FConcertServer::HandleCopySessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_ArchiveSessionRequest, FConcertAdmin_ArchiveSessionResponse>(this, &FConcertServer::HandleArchiveSessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_RenameSessionRequest, FConcertAdmin_RenameSessionResponse>(this, &FConcertServer::HandleRenameSessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_DeleteSessionRequest, FConcertAdmin_DeleteSessionResponse>(this, &FConcertServer::HandleDeleteSessionRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_BatchDeleteSessionRequest, FConcertAdmin_BatchDeleteSessionResponse>(this, &FConcertServer::HandleBatchDeleteSessionRequest);

		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetAllSessionsRequest, FConcertAdmin_GetAllSessionsResponse>(this, &FConcertServer::HandleGetAllSessionsRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetLiveSessionsRequest, FConcertAdmin_GetSessionsResponse>(this, &FConcertServer::HandleGetLiveSessionsRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetArchivedSessionsRequest, FConcertAdmin_GetSessionsResponse>(this, &FConcertServer::HandleGetArchivedSessionsRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetSessionClientsRequest, FConcertAdmin_GetSessionClientsResponse>(this, &FConcertServer::HandleGetSessionClientsRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetSessionActivitiesRequest, FConcertAdmin_GetSessionActivitiesResponse>(this, &FConcertServer::HandleGetSessionActivitiesRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_MountSessionRepositoryRequest, FConcertAdmin_MountSessionRepositoryResponse>(this, &FConcertServer::HandleMountSessionRepositoryRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_GetSessionRepositoriesRequest, FConcertAdmin_GetSessionRepositoriesResponse>(this, &FConcertServer::HandleGetSessionRepositoriesRequest);
		ServerAdminEndpoint->RegisterRequestHandler<FConcertAdmin_DropSessionRepositoriesRequest, FConcertAdmin_DropSessionRepositoriesResponse>(this, &FConcertServer::HandleDropSessionRepositoriesRequest);

		// Perform maintenance tasks on the session repositories database.
		{
			FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());

			// Load the file containing the repositories.
			FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
			ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);

			// Unmap repositories that doesn't exist anymore on disk (were deleted manually).
			int RemovedNum = SessionRepositoryDb.Repositories.RemoveAll([](const FConcertServerSessionRepository& RemoveCandidate)
			{
				if (!RemoveCandidate.RepositoryRootDir.IsEmpty()) // Under a single standard root?
				{
					FString Pathname = RemoveCandidate.RepositoryRootDir / RemoveCandidate.RepositoryId.ToString();
					return !IFileManager::Get().DirectoryExists(*Pathname);
				}
				return false; // Not under a single root (Multi-User backward compatibility mode) leave it.
			});
			if (RemovedNum)
			{
				ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
			}

			// Walk the root directory containing the server managed repositories and find those that aren't mapped anymore.
			TArray<FString> ExpiredDirectories;
			IFileManager::Get().IterateDirectory(*GetSessionRepositoriesRootDir(), [this, &SessionRepositoryDb, &ExpiredDirectories](const TCHAR* Pathname, bool bIsDirectory)
			{
				if (bIsDirectory)
				{
					// Check if the repository is still mapped.
					FString RootReposDir = GetSessionRepositoriesRootDir();
					if (!SessionRepositoryDb.Repositories.ContainsByPredicate([&RootReposDir, Pathname](const FConcertServerSessionRepository& Repository) { return  RootReposDir / Repository.RepositoryId.ToString() == Pathname; }))
					{
						ExpiredDirectories.Emplace(Pathname); // The visited directory was not found in the list of mapped repositories.
					}
				}
				return true;
			});

			// Delete the directories that are not mapped anymore.
			for (const FString& Dir : ExpiredDirectories)
			{
				FGuid Dummy;
				if (FGuid::Parse(FPaths::GetPathLeaf(Dir), Dummy)) // Ensure the directory name is a GUID as the repositories base dir is the repository ID.
				{
					ConcertUtil::DeleteDirectoryTree(*Dir);
				}
			}
		}

		// Try to mount the default session repository configured (if one is configured) to lock the non-sharable session files away from concurrent processes.
		MountDefaultSessionRepository(Settings.Get());

		OnConcertServerStartupDelegate.Broadcast();
	}
}

void FConcertServer::Shutdown()
{
	// Server Query
	if (ServerAdminEndpoint.IsValid())
	{
		// Discovery
		ServerAdminEndpoint->UnsubscribeEventHandler<FConcertAdmin_DiscoverServersEvent>();

		// Session connection
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_CreateSessionRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_FindSessionRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_CopySessionRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_ArchiveSessionRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_RenameSessionRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_DeleteSessionRequest>();

		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetAllSessionsRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetLiveSessionsRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetArchivedSessionsRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetSessionClientsRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetSessionActivitiesRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_MountSessionRepositoryRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_GetSessionRepositoriesRequest>();
		ServerAdminEndpoint->UnregisterRequestHandler<FConcertAdmin_DropSessionRepositoriesRequest>();

		ServerAdminEndpoint.Reset();
	}

	// Destroy the live sessions
	{
		TArray<FGuid> LiveSessionIds;
		LiveSessions.GetKeys(LiveSessionIds);
		for (const FGuid& LiveSessionId : LiveSessionIds)
		{
			bool bDeleteSessionData = false;
			if (Settings->bAutoArchiveOnShutdown)
			{
				bDeleteSessionData = ArchiveLiveSession(LiveSessionId, FString(), AutoArchiveSessionFilter).IsValid();
			}
			DestroyLiveSession(LiveSessionId, bDeleteSessionData);
		}
		LiveSessions.Reset();
	}

	// Destroy the archived sessions
	{
		TArray<FGuid> ArchivedSessionIds;
		ArchivedSessions.GetKeys(ArchivedSessionIds);
		for (const FGuid& ArchivedSessionId : ArchivedSessionIds)
		{
			DestroyArchivedSession(ArchivedSessionId, /*bDeleteSessionData*/false);
		}
		ArchivedSessions.Reset();
	}

	// Concurrent server instances may fight to get ownership of the info file.
	FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());

	// Load the file containing the instance info.
	FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
	ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);
	bool bSaveRepositoryDatabase = false;

	// Unmount all repositories mounted by this instance.
	int32 ProcessId = FPlatformProcess::GetCurrentProcessId();
	for (FConcertServerSessionRepository& Repository : SessionRepositoryDb.Repositories)
	{
		if (Repository.bMounted && Repository.ProcessId == ProcessId)
		{
			Repository.bMounted = false;
			Repository.ProcessId = 0;
			bSaveRepositoryDatabase = true;
		}
	}

	if (bSaveRepositoryDatabase)
	{
		ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
	}
}

FGuid FConcertServer::GetLiveSessionIdByName(const FString& InName) const
{
	for (const auto& LiveSessionPair : LiveSessions)
	{
		if (LiveSessionPair.Value->GetName() == InName)
		{
			return LiveSessionPair.Key;
		}
	}
	return FGuid();
}

FGuid FConcertServer::GetArchivedSessionIdByName(const FString& InName) const
{
	for (const auto& ArchivedSessionPair : ArchivedSessions)
	{
		if (ArchivedSessionPair.Value.SessionName == InName)
		{
			return ArchivedSessionPair.Key;
		}
	}
	return FGuid();
}

FConcertSessionInfo FConcertServer::CreateSessionInfo() const
{
	FConcertSessionInfo SessionInfo;
	SessionInfo.ServerInstanceId = ServerInfo.InstanceInfo.InstanceId;
	SessionInfo.OwnerInstanceId = ServerInfo.InstanceInfo.InstanceId;
	SessionInfo.OwnerUserName = FApp::GetSessionOwner();
	SessionInfo.OwnerDeviceName = FPlatformProcess::ComputerName();
	SessionInfo.SessionId = FGuid::NewGuid();
	return SessionInfo;
}

TSharedPtr<IConcertServerSession> FConcertServer::CreateSession(const FConcertSessionInfo& SessionInfo, FText& OutFailureReason)
{
	if (!SessionInfo.SessionId.IsValid() || SessionInfo.SessionName.IsEmpty())
	{
		OutFailureReason = LOCTEXT("Error_CreateSession_EmptySessionIdOrName", "Empty session ID or name");
		UE_LOGF(LogConcert, Error, "An attempt to create a session was made, but the session info was missing an ID or name!");
		return nullptr;
	}

	if (!Settings->ServerSettings.bIgnoreSessionSettingsRestriction && SessionInfo.VersionInfos.Num() == 0)
	{
		OutFailureReason = LOCTEXT("Error_CreateSession_EmptyVersionInfo", "Empty version info");
		UE_LOGF(LogConcert, Error, "An attempt to create a session was made, but the session info was missing version info!");
		return nullptr;
	}

	if (LiveSessions.Contains(SessionInfo.SessionId))
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_CreateSession_AlreadyExists", "Session '{0}' already exists"), FText::AsCultureInvariant(SessionInfo.SessionId.ToString()));
		UE_LOGF(LogConcert, Error, "An attempt to create a session with ID '%ls' was made, but that session already exists!", *SessionInfo.SessionId.ToString());
		return nullptr;
	}

	if (GetLiveSessionIdByName(SessionInfo.SessionName).IsValid())
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_CreateSession_AlreadyExists", "Session '{0}' already exists"), FText::AsCultureInvariant(SessionInfo.SessionName));
		UE_LOGF(LogConcert, Error, "An attempt to create a session with name '%ls' was made, but that session already exists!", *SessionInfo.SessionName);
		return nullptr;
	}

	// If the default session repository is not set, check if one is configured and try to mount it. This may be a time-costly operation. This is to addresses the case where a user
	// has/had two concurrent Multi-User servers using the same sessions directories without noticing and fail to create a session on the newest server instance because the folder is/was
	// locked by the older instance when the new one started.
	if (!DefaultSessionRepository && !MountDefaultSessionRepository(Settings.Get()))
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_CreateSession_NoRepository", "Session '{0}' could not be created. The default repository used to store sessions files is not mounted. Reason: {1}"), FText::AsCultureInvariant(SessionInfo.SessionName), DefaultSessionRepositoryStatus);
		UE_LOGF(LogConcert, Error, "An attempt to create a session with name '%ls' was made, but the server did not have any repository mounted to store it! The repository may already be mounted by another process.", *SessionInfo.SessionName);
		return nullptr;
	}

	return CreateLiveSession(SessionInfo, DefaultSessionRepository.GetValue());
}

TSharedPtr<IConcertServerSession> FConcertServer::RestoreSession(const FGuid& SessionId, const FConcertSessionInfo& SessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason)
{
	if (ArchivedSessions.Contains(SessionId))
	{
		return CopySession(SessionId, SessionInfo, SessionFilter, OutFailureReason);
	}

	OutFailureReason = FText::Format(LOCTEXT("Error_RestoreSession_NotFound", "Session '{0}' not found"), FText::AsCultureInvariant(SessionId.ToString()));
	UE_LOGF(LogConcert, Error, "An attempt to restore session '%ls' was made, but that session could not be found!", *SessionId.ToString());
	return nullptr;
}

TSharedPtr<IConcertServerSession> FConcertServer::CopySession(const FGuid& SrcSessionId, const FConcertSessionInfo& NewSessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason)
{
	if (!NewSessionInfo.SessionId.IsValid() || NewSessionInfo.SessionName.IsEmpty())
	{
		OutFailureReason = LOCTEXT("Error_CopySession_EmptySessionIdOrName", "Empty session ID or name");
		UE_LOGF(LogConcert, Error, "An attempt to copy a session was made, but the session info was missing an ID or name!");
		return nullptr;
	}
	else if (!Settings->ServerSettings.bIgnoreSessionSettingsRestriction && NewSessionInfo.VersionInfos.Num() == 0)
	{
		OutFailureReason = LOCTEXT("Error_CopySession_EmptyVersionInfo", "Empty version info");
		UE_LOGF(LogConcert, Error, "An attempt to copy a session was made, but the session info was missing version info!");
		return nullptr;
	}
	else if (LiveSessions.Contains(NewSessionInfo.SessionId))
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_CopySession_AlreadyExists", "Session '{0}' already exists"), FText::AsCultureInvariant(NewSessionInfo.SessionId.ToString()));
		UE_LOGF(LogConcert, Error, "An attempt to copy a session with ID '%ls' was made, but that session already exists!", *NewSessionInfo.SessionId.ToString());
		return nullptr;
	}
	else if (GetLiveSessionIdByName(NewSessionInfo.SessionName).IsValid())
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_CopySession_AlreadyExists", "Session '{0}' already exists"), FText::AsCultureInvariant(NewSessionInfo.SessionName));
		UE_LOGF(LogConcert, Error, "An attempt to copy a session with name '%ls' was made, but that session already exists!", *NewSessionInfo.SessionName);
		return nullptr;
	}
	else if (ArchivedSessions.Contains(SrcSessionId))
	{
		return RestoreArchivedSession(SrcSessionId, NewSessionInfo, SessionFilter, OutFailureReason);
	}
	else if (TSharedPtr<IConcertServerSession> LiveSession = LiveSessions.FindRef(SrcSessionId))
	{
		// Copy the live session in the default repository (where new sessions should be created), unless it is unset.
		const FConcertSessionInfo& LiveSessionInfo = LiveSession->GetSessionInfo();
		const FConcertServerSessionRepository& CopySessionRepository = DefaultSessionRepository.IsSet() ? DefaultSessionRepository.GetValue() : GetSessionRepository(LiveSession->GetSessionInfo().SessionId);
		if (EventSink->CopySession(*this, LiveSession.ToSharedRef(), CopySessionRepository.GetSessionWorkingDir(NewSessionInfo.SessionId), SessionFilter))
		{
			UE_LOGF(LogConcert, Display, "Live session '%ls' (%ls) was copied as '%ls' (%ls)", *LiveSessionInfo.SessionName, *LiveSessionInfo.SessionId.ToString(), *NewSessionInfo.SessionName, *NewSessionInfo.SessionId.ToString());
			return CreateLiveSession(NewSessionInfo, CopySessionRepository);
		}
		else
		{
			UE_LOGF(LogConcert, Error, "An attempt to copy a session '%ls' was made, but failed!", *SrcSessionId.ToString());
			return nullptr;
		}
	}

	OutFailureReason = FText::Format(LOCTEXT("Error_CopySession_NotFound", "Session '{0}' not found"), FText::AsCultureInvariant(SrcSessionId.ToString()));
	UE_LOGF(LogConcert, Error, "An attempt to copy a session '%ls' was made, but that session could not be found!", *SrcSessionId.ToString());
	return nullptr;
}

void FConcertServer::RecoverSessions(const FConcertServerSessionRepository& InRepository, bool bCleanupExpiredSessions)
{
	// Find any existing live sessions to automatically restore when recovering from an improper server shutdown
	TArray<FConcertSessionInfo> LiveSessionInfos;
	TArray<FDateTime> LiveSessionCreationTimes;
	EventSink->GetSessionsFromPath(*this, InRepository.WorkingDir, LiveSessionInfos, &LiveSessionCreationTimes);
	UpdateLastModified(LiveSessionInfos, LiveSessionCreationTimes);
	
	// Restore any existing live sessions
	for (FConcertSessionInfo& LiveSessionInfo : LiveSessionInfos)
	{
		// Update the session info with new server info
		LiveSessionInfo.ServerInstanceId = ServerInfo.InstanceInfo.InstanceId;
		if (!LiveSessions.Contains(LiveSessionInfo.SessionId) && !GetLiveSessionIdByName(LiveSessionInfo.SessionName).IsValid() && CreateLiveSession(LiveSessionInfo, InRepository))
		{
			UE_LOGF(LogConcert, Display, "Live session '%ls' (%ls) was recovered.", *LiveSessionInfo.SessionName, *LiveSessionInfo.SessionId.ToString());
		}
	}

	if (bCleanupExpiredSessions && Settings->NumSessionsToKeep == 0)
	{
		ConcertUtil::DeleteDirectoryTree(*InRepository.SavedDir);
	}
	else
	{
		// Find any existing archived sessions
		TArray<FConcertSessionInfo> ArchivedSessionInfos;
		TArray<FDateTime> ArchivedSessionCreationTimes;

		// In theory, archives are immutable, but the server will end up touching the files and change the 'modification time'. Ensure to look at 'creation time'.
		EventSink->GetSessionsFromPath(*this, InRepository.SavedDir, ArchivedSessionInfos, &ArchivedSessionCreationTimes);
		check(ArchivedSessionInfos.Num() == ArchivedSessionCreationTimes.Num());
		UpdateLastModified(ArchivedSessionInfos, ArchivedSessionCreationTimes);
		
		// Trim the oldest archived sessions.
		if (bCleanupExpiredSessions && Settings->NumSessionsToKeep > 0 && ArchivedSessionInfos.Num() > Settings->NumSessionsToKeep)
		{
			typedef TTuple<int32, FDateTime> FSavedSessionInfo;

			// Build the list of sorted session
			TArray<FSavedSessionInfo> SortedSessions;
			for (int32 LiveSessionInfoIndex = 0; LiveSessionInfoIndex < ArchivedSessionInfos.Num(); ++LiveSessionInfoIndex)
			{
				SortedSessions.Add(MakeTuple(LiveSessionInfoIndex, ArchivedSessionCreationTimes[LiveSessionInfoIndex]));
			}
			SortedSessions.Sort([](const FSavedSessionInfo& InOne, const FSavedSessionInfo& InTwo)
			{
				return InOne.Value < InTwo.Value;
			});

			// Keep the most recent sessions
			TArray<FConcertSessionInfo> ArchivedSessionsToKeep;
			{
				const int32 FirstSortedSessionIndexToKeep = SortedSessions.Num() - Settings->NumSessionsToKeep;
				for (int32 SortedSessionIndex = FirstSortedSessionIndexToKeep; SortedSessionIndex < SortedSessions.Num(); ++SortedSessionIndex)
				{
					ArchivedSessionsToKeep.Add(ArchivedSessionInfos[SortedSessions[SortedSessionIndex].Key]);
				}
				SortedSessions.RemoveAt(FirstSortedSessionIndexToKeep, Settings->NumSessionsToKeep, EAllowShrinking::No);
			}

			// Remove the oldest sessions
			for (const FSavedSessionInfo& SortedSession : SortedSessions)
			{
				ConcertUtil::DeleteDirectoryTree(*InRepository.GetSessionSavedDir(ArchivedSessionInfos[SortedSession.Key].SessionId));
			}

			// Update the list of sessions to restore
			ArchivedSessionInfos = MoveTemp(ArchivedSessionsToKeep);
			ArchivedSessionCreationTimes.Reset();
		}

		// Create any existing archived sessions
		for (FConcertSessionInfo& ArchivedSessionInfo : ArchivedSessionInfos)
		{
			// Update the session info with new server info
			ArchivedSessionInfo.ServerInstanceId = ServerInfo.InstanceInfo.InstanceId;
			if (!ArchivedSessions.Contains(ArchivedSessionInfo.SessionId) && !GetArchivedSessionIdByName(ArchivedSessionInfo.SessionName).IsValid() && CreateArchivedSession(ArchivedSessionInfo))
			{
				UE_LOGF(LogConcert, Display, "Archived session '%ls' (%ls) was discovered.", *ArchivedSessionInfo.SessionName, *ArchivedSessionInfo.SessionId.ToString());
			}
		}
	}
}

void FConcertServer::UpdateLastModified(TArray<FConcertSessionInfo>& SessionInfos, const TArray<FDateTime>& SessionCreationTimes)
{
	for (int32 i = 0; i < SessionInfos.Num(); ++i)
	{
		SessionInfos[i].SetLastModified(SessionCreationTimes[i]);
	}
}

void FConcertServer::ArchiveOfflineSessions(const FConcertServerSessionRepository& InRepository)
{
	// Find existing live session files to automatically archive them when recovering from an improper server shutdown.
	TArray<FConcertSessionInfo> LiveSessionInfos;
	EventSink->GetSessionsFromPath(*this, InRepository.WorkingDir, LiveSessionInfos);

	// Migrate the live sessions files into their archived form.
	for (FConcertSessionInfo& LiveSessionInfo : LiveSessionInfos)
	{
		LiveSessionInfo.ServerInstanceId = ServerInfo.InstanceInfo.InstanceId;
		FConcertSessionInfo ArchivedSessionInfo = LiveSessionInfo;
		ArchivedSessionInfo.SessionId = FGuid::NewGuid();
		ArchivedSessionInfo.SessionName = ConcertServerUtil::GetArchiveName(LiveSessionInfo.SessionName, LiveSessionInfo.Settings);
		ArchivedSessionInfo.SetLastModifiedToNow();

		if (EventSink->ArchiveSession(*this, InRepository.GetSessionWorkingDir(LiveSessionInfo.SessionId), InRepository.GetSessionSavedDir(ArchivedSessionInfo.SessionId), ArchivedSessionInfo, AutoArchiveSessionFilter))
		{
			UE_LOGF(LogConcert, Display, "Deleting %ls", *InRepository.GetSessionWorkingDir(LiveSessionInfo.SessionId));
			ConcertUtil::DeleteDirectoryTree(*InRepository.GetSessionWorkingDir(LiveSessionInfo.SessionId));
			UE_LOGF(LogConcert, Display, "Live session '%ls' (%ls) was archived on reboot.", *LiveSessionInfo.SessionName, *LiveSessionInfo.SessionId.ToString());
		}
	}
}

FGuid FConcertServer::ArchiveSession(const FGuid& SessionId, const FString& ArchiveNameOverride, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason, FGuid ArchiveSessionIdOverride)
{
	if (GetArchivedSessionIdByName(ArchiveNameOverride).IsValid())
	{
		OutFailureReason = FText::Format(LOCTEXT("Error_ArchiveSession_AlreadyExists", "Archived session '{0}' already exists"), FText::AsCultureInvariant(ArchiveNameOverride));
		return FGuid();
	}

	const FGuid ArchivedSessionId = ArchiveLiveSession(SessionId, ArchiveNameOverride, SessionFilter, MoveTemp(ArchiveSessionIdOverride));
	if (!ArchivedSessionId.IsValid())
	{
		OutFailureReason = LOCTEXT("Error_ArchiveSession_FailedToCopy", "Could not copy session data to the archive");
		return FGuid();
	}

	return ArchivedSessionId;
}

bool FConcertServer::ExportSession(const FGuid& SessionId, const FConcertSessionFilter& SessionFilter, const FString& DestDir, bool bAnonymizeData, FText& OutFailureReason)
{
	return EventSink->ExportSession(*this, SessionId, DestDir, SessionFilter, bAnonymizeData);
}

bool FConcertServer::RenameSession(const FGuid& SessionId, const FString& NewName, FText& OutFailureReason)
{
	// NOTE: This function is exposed to the server internals and should not be directly called by connected clients. Clients
	//       send requests (see HandleRenameSessionRequest()). When this function is called, the caller is treated as an 'Admin'.

	FConcertAdmin_RenameSessionRequest Request;
	Request.SessionId = SessionId;
	Request.NewName = NewName;
	Request.UserName = TEXT("Admin");
	Request.DeviceName = FString();
	bool bCheckPermissions = false; // The caller is expected to be a server Admin, bypass permissions.

	FConcertAdmin_RenameSessionResponse Response = RenameSessionInternal(Request, bCheckPermissions);
	OutFailureReason = Response.Reason;
	return Response.ResponseCode == EConcertResponseCode::Success;
}

bool FConcertServer::DestroySession(const FGuid& SessionId, FText& OutFailureReason)
{
	// NOTE: This function is exposed to the server internals and should not be directly called by connected clients. Clients
	//       send requests (see HandleDeleteSessionRequest()). When this function is called, the caller is treated as an 'Admin'.

	FConcertAdmin_DeleteSessionRequest Request;
	Request.SessionId = SessionId;
	Request.UserName = TEXT("Admin");
	Request.DeviceName = FString();
	bool bCheckPermissions = false; // The caller is expected to be a server Admin, bypass permissions.

	FConcertAdmin_DeleteSessionResponse Response = DeleteSessionInternal(Request, bCheckPermissions);
	OutFailureReason = Response.Reason;
	return Response.ResponseCode == EConcertResponseCode::Success;
}

FOnConcertServerSessionStartup& FConcertServer::OnConcertServerSessionStartup()
{
	return OnConcertServerSessionStartupDelegate;
}

FOnConcertServerStartup& FConcertServer::OnConcertServerStartup()
{
	return OnConcertServerStartupDelegate;
}

FOnConcertParticipantCanJoinSession& FConcertServer::OnConcertParticipantCanJoinSession()
{
	return OnConcertParticipantCanJoinSessionDelegate;
}

TArray<FConcertSessionInfo> FConcertServer::GetLiveSessionInfos() const
{
	TArray<FConcertSessionInfo> SessionsInfo;
	SessionsInfo.Reserve(LiveSessions.Num());
	for (auto& SessionPair : LiveSessions)
	{
		SessionsInfo.Add(SessionPair.Value->GetSessionInfo());
	}
	return SessionsInfo;
}

TArray<FConcertSessionInfo> FConcertServer::GetArchivedSessionInfos() const
{
	TArray<FConcertSessionInfo> SessionsInfo;
	SessionsInfo.Reserve(ArchivedSessions.Num());
	for (auto& SessionPair : ArchivedSessions)
	{
		SessionsInfo.Add(SessionPair.Value);
	}
	return SessionsInfo;
}

TArray<TSharedPtr<IConcertServerSession>> FConcertServer::GetLiveSessions() const
{
	TArray<TSharedPtr<IConcertServerSession>> SessionsArray;
	SessionsArray.Reserve(LiveSessions.Num());
	for (auto& SessionPair : LiveSessions)
	{
		SessionsArray.Add(SessionPair.Value);
	}
	return SessionsArray;
}

TSharedPtr<IConcertServerSession> FConcertServer::GetLiveSession(const FGuid& SessionId) const
{
	return LiveSessions.FindRef(SessionId);
}

TOptional<FConcertSessionInfo> FConcertServer::GetArchivedSessionInfo(const FGuid& SessionId) const
{
	const FConcertSessionInfo* SessionInfo = ArchivedSessions.Find(SessionId);
	return SessionInfo ? *SessionInfo : TOptional<FConcertSessionInfo>{};
}

const FString& FConcertServer::GetSessionRepositoriesRootDir() const
{
	return SessionRepositoryRootDir;
}

const FConcertServerSessionRepository& FConcertServer::GetSessionRepository(const FGuid& SessionId) const
{
	const FConcertServerSessionRepository* SessionRepository = MountedSessionRepositories.FindByPredicate([SessionId](const FConcertServerSessionRepository& MountedRepository)
	{
		return IFileManager::Get().DirectoryExists(*MountedRepository.GetSessionWorkingDir(SessionId)) || IFileManager::Get().DirectoryExists(*MountedRepository.GetSessionSavedDir(SessionId));
	});

	check(SessionRepository); // If the session is in memory, its repository must be mounted.
	return *SessionRepository;
}

FString FConcertServer::GetSessionSavedDir(const FGuid& SessionId) const
{
	return GetSessionRepository(SessionId).GetSessionSavedDir(SessionId);
}

FString FConcertServer::GetSessionWorkingDir(const FGuid& SessionId) const
{
	return GetSessionRepository(SessionId).GetSessionWorkingDir(SessionId);
}

EConcertSessionRepositoryMountResponseCode FConcertServer::MountSessionRepository(FConcertServerSessionRepository Repository, bool bCreateIfNotExist, bool bCleanWorkingDir, bool bCleanExpiredSessions, bool bSearchByPaths, bool bAsDefault)
{
	EConcertSessionRepositoryMountResponseCode MountStatus = EConcertSessionRepositoryMountResponseCode::Mounted;
	FText MountStatusText = LOCTEXT("SessionRepository_Mounted", "Repository mounted.");
	bool bAlreadyMountedByThisProcess = false;

	// Exclusive access scope to the session repository db.
	{
		// Load the file containing the instance/repository info.
		FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());
		FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
		ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);

		check(!Repository.bMounted && Repository.ProcessId == 0); // Should not be mounted.

		// Check if the repository can be found in the database.
		if (FConcertServerSessionRepository* ExistingRepository = SessionRepositoryDb.Repositories.FindByPredicate(
			[&Repository, bSearchByPaths](const FConcertServerSessionRepository& Candidate){ return bSearchByPaths ? Candidate.WorkingDir == Repository.WorkingDir && Candidate.SavedDir == Repository.SavedDir : Candidate.RepositoryId == Repository.RepositoryId; }))
		{
			if (!ExistingRepository->bMounted || !FPlatformProcess::IsApplicationRunning(ExistingRepository->ProcessId)) // Not mounted or mounted by a dead process.
			{
				check(Repository.RepositoryRootDir == ExistingRepository->RepositoryRootDir) // The client changed the root dir?
				ExistingRepository->bMounted = true;
				ExistingRepository->ProcessId = FPlatformProcess::GetCurrentProcessId();
				Repository = *ExistingRepository;
				MountedSessionRepositories.Add(Repository);
				ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
			}
			else if (ExistingRepository->ProcessId == FPlatformProcess::GetCurrentProcessId() &&
				MountedSessionRepositories.ContainsByPredicate([&Repository](const FConcertServerSessionRepository& MatchCandidate){ return MatchCandidate.RepositoryId == Repository.RepositoryId; })) // Already mounted by this process?
			{
				UE_LOGF(LogConcert, Display, "Remounted repository %ls. The repository is already mounted by this process.", *Repository.RepositoryId.ToString());
				bAlreadyMountedByThisProcess = true; // Already mounted by this process, don't process the session files again.
			}
			else
			{
				UE_LOGF(LogConcert, Warning, "Failed to mount repository %ls. The repository is already mounted by another process.", *Repository.RepositoryId.ToString());
				MountStatus = EConcertSessionRepositoryMountResponseCode::AlreadyMounted; // Already mounted by another process, cannot mount it, the files are not shareable.
				MountStatusText = LOCTEXT("SessionRepository_AlreadyMounted", "Repository locked by another process.");
			}
		}
		else if (bCreateIfNotExist)
		{
			Repository.bMounted = true;
			Repository.ProcessId = FPlatformProcess::GetCurrentProcessId();
			MountedSessionRepositories.Add(Repository);
			SessionRepositoryDb.Repositories.Add(Repository);
			ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "Failed to mount repository %ls. The repository was not found.", *Repository.RepositoryId.ToString());
			MountStatus = EConcertSessionRepositoryMountResponseCode::NotFound;
			MountStatusText = LOCTEXT("SessionRepository_NotFound", "Repository not found.");
		}
	}

	// Should the mounted repository be used as default?
	if (bAsDefault)
	{
		if (MountStatus == EConcertSessionRepositoryMountResponseCode::Mounted)
		{
			DefaultSessionRepository = Repository;
			UE_LOGF(LogConcert, Display, "Default session repository %ls set successfully.", *Repository.RepositoryId.ToString());
		}
		else
		{
			DefaultSessionRepository.Reset();
			UE_LOGF(LogConcert, Warning, "Default session repository %ls failed to mount.", *Repository.RepositoryId.ToString());
		}
		DefaultSessionRepositoryStatus = MountStatusText;
	}

	// Should the sessions in the repository processed?
	if (MountStatus == EConcertSessionRepositoryMountResponseCode::Mounted && !bAlreadyMountedByThisProcess)
	{
		// Process the sessions in the repository.
		if (bCleanWorkingDir)
		{
			ConcertUtil::DeleteDirectoryTree(*Repository.WorkingDir);
		}
		else if (Settings->bAutoArchiveOnReboot) // Honor the auto-archive settings when mounting a new repository.
		{
			// Migrate live sessions files (session is not restored yet) to its archive form and directory.
			ArchiveOfflineSessions(Repository);
		}

		// Reload the archived/live sessions and possibly rotate the list of archives to prevent having too many of them.
		RecoverSessions(Repository, bCleanExpiredSessions);
	}

	return MountStatus;
}

bool FConcertServer::UnmountSessionRepository(const FGuid& RepositoryId, bool bDropped)
{
	// Search the repository in the list of mounted repositories.
	int32 Index = MountedSessionRepositories.IndexOfByPredicate([&RepositoryId](const FConcertServerSessionRepository& MatchCandidate) { return RepositoryId == MatchCandidate.RepositoryId; });
	if (Index == INDEX_NONE)
	{
		return false; // Not mounted by this process.
	}

	FConcertServerSessionRepository& Repository = MountedSessionRepositories[Index];
	check(Repository.bMounted); // Must be mounted if present in the 'mounted' list.
	check(Repository.ProcessId == FPlatformProcess::GetCurrentProcessId()); // Must be mounted by this process to be in the list.

	// Unload the live sessions hosted in that repository.
	TArray<FGuid> LiveSessionIds;
	LiveSessions.GetKeys(LiveSessionIds);
	for (const FGuid& LiveSessionId : LiveSessionIds)
	{
		const FConcertServerSessionRepository& SessionRepository = GetSessionRepository(LiveSessionId);
		if (SessionRepository.RepositoryId == RepositoryId)
		{
			DestroyLiveSession(LiveSessionId, /*bDeleteSessionData*/bDropped);
		}
	}

	// Unload the archived sessions hosted in that repository.
	TArray<FGuid> ArchivedSessionIds;
	ArchivedSessions.GetKeys(ArchivedSessionIds);
	for (const FGuid& ArchivedSessionId : ArchivedSessionIds)
	{
		const FConcertServerSessionRepository& SessionRepository = GetSessionRepository(ArchivedSessionId);
		if (SessionRepository.RepositoryId == RepositoryId)
		{
			DestroyArchivedSession(ArchivedSessionId, /*bDeleteSessionData*/bDropped);
		}
	}

	if (DefaultSessionRepository && DefaultSessionRepository->RepositoryId == RepositoryId)
	{
		DefaultSessionRepository.Reset(); // Will not be able to create new sessions until a mounted repository is set as default.
		DefaultSessionRepositoryStatus = LOCTEXT("SessionRepository_Unmounted", "Repository unmounted.");
		UE_LOGF(LogConcert, Warning, "Default repository %ls unmounted. No session will be created until a mounted repository is set as default", *RepositoryId.ToString());
	}
	else
	{
		UE_LOGF(LogConcert, Display, "Repository %ls unmounted.", *RepositoryId.ToString())
	}

	if (bDropped && !Repository.RepositoryRootDir.IsEmpty()) // When dropped, the repository can be deleted if it has the standard root structure.
	{
		FString RepositoryDir = Repository.RepositoryRootDir / Repository.RepositoryId.ToString();
		if (ConcertUtil::DeleteDirectoryTree(*RepositoryDir))
		{
			UE_LOGF(LogConcert, Display, "Repository %ls deleted.", *Repository.RepositoryId.ToString())
		}
	}

	// Remove the repository from the of mounted repository list.
	MountedSessionRepositories.RemoveAt(Index);

	// Update the repository database file
	{
		FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());
		FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
		ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);
		if (bDropped)
		{
			SessionRepositoryDb.Repositories.RemoveAll([&RepositoryId](const FConcertServerSessionRepository& RemoveCandidate) { return RepositoryId == RemoveCandidate.RepositoryId; });
		}
		else if (FConcertServerSessionRepository* UnmountedRepo = SessionRepositoryDb.Repositories.FindByPredicate([&RepositoryId](const FConcertServerSessionRepository& MatchRepository) { return RepositoryId == MatchRepository.RepositoryId; }))
		{
			UnmountedRepo->bMounted = false;
			UnmountedRepo->ProcessId = 0;
		}
		ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
	}

	return true;
}

bool FConcertServer::MountDefaultSessionRepository(const UConcertServerConfig* ServerConfig)
{
	if (DefaultSessionRepository)
	{
		return true; // A default session repository is already mounted.
	}

	// If the server was configured to use a custom working/archive dir, create a corresponding repository and try to mount it.
	if (!ServerConfig->WorkingDir.IsEmpty() || !ServerConfig->ArchiveDir.IsEmpty())
	{
		FConcertServerSessionRepository Repository(Role, FGuid::NewGuid(), ServerConfig->WorkingDir, ServerConfig->ArchiveDir);
		return MountSessionRepository(MoveTemp(Repository), /*bCreateIfNotExist*/true, ServerConfig->bCleanWorkingDir, /*bCleanupExpiredSession*/true, /*bSearchByPath*/true, /*bAsDefault*/true) == EConcertSessionRepositoryMountResponseCode::Mounted;
	}
	// If the server was configured to mount a default server managed repository.
	else if (ServerConfig->bMountDefaultSessionRepository)
	{
		FConcertServerSessionRepository Repository(GetSessionRepositoriesRootDir(), FGuid()); // Invalid GUID is used for the default server repository.
		return MountSessionRepository(MoveTemp(Repository), /*bCreateIfNotExist*/true, ServerConfig->bCleanWorkingDir, /*bCleanupExpiredSession*/true, /*bSearchByPath*/false, /*bAsDefault*/true) == EConcertSessionRepositoryMountResponseCode::Mounted;
	}

	return false; // No session repository was mounted as default.
}

void FConcertServer::HandleDiscoverServersEvent(const FConcertMessageContext& Context)
{
	const FConcertAdmin_DiscoverServersEvent* Message = Context.GetMessage<FConcertAdmin_DiscoverServersEvent>();

	if (Message->ConcertProtocolVersion == EConcertMessageVersion::LatestVersion && 
		ServerAdminEndpoint.IsValid() && 
		Message->RequiredRole == Role &&
		Message->RequiredVersion == VERSION_STRINGIFY(ENGINE_MAJOR_VERSION) TEXT(".") VERSION_STRINGIFY(ENGINE_MINOR_VERSION))
	{
		if (Settings->AuthorizedClientKeys.Num() == 0 || Settings->AuthorizedClientKeys.Contains(Message->ClientAuthenticationKey)) // Can the client discover this server?
		{
			FConcertAdmin_ServerDiscoveredEvent DiscoveryInfo;
			DiscoveryInfo.ConcertProtocolVersion = EConcertMessageVersion::LatestVersion;
			DiscoveryInfo.ServerName = ServerInfo.ServerName;
			DiscoveryInfo.InstanceInfo = ServerInfo.InstanceInfo;
			DiscoveryInfo.ServerFlags = ServerInfo.ServerFlags;
			ServerAdminEndpoint->SendEvent(DiscoveryInfo, Context.SenderConcertEndpointId);
		}
	}
}

TFuture<FConcertAdmin_MountSessionRepositoryResponse> FConcertServer::HandleMountSessionRepositoryRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_MountSessionRepositoryRequest* Message = Context.GetMessage<FConcertAdmin_MountSessionRepositoryRequest>();

	FConcertAdmin_MountSessionRepositoryResponse ResponseData;
	if (Message->RepositoryRootDir.IsEmpty()) // Use the server configured repository root dir?
	{
		FConcertServerSessionRepository Repository(GetSessionRepositoriesRootDir(), Message->RepositoryId);
		ResponseData.MountStatus = MountSessionRepository(MoveTemp(Repository), Message->bCreateIfNotExist, /*bCleanWorkingDir*/false, /*bCleanExpiredSessions*/false, /*bSearchByPaths*/false, Message->bAsServerDefault);
	}
	else // Use the client supplied repository root dir.
	{
		FConcertServerSessionRepository Repository(Message->RepositoryRootDir, Message->RepositoryId);
		ResponseData.MountStatus = MountSessionRepository(MoveTemp(Repository), Message->bCreateIfNotExist, /*bCleanWorkingDir*/false, /*bCleanExpiredSessions*/false, /*bSearchByPaths*/false, Message->bAsServerDefault);
	}

	return FConcertAdmin_MountSessionRepositoryResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_GetSessionRepositoriesResponse> FConcertServer::HandleGetSessionRepositoriesRequest(const FConcertMessageContext& Context)
{
	// Prevent concurrent access to the instance file.
	FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());

	// Load the global database containing all known repositories.
	FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
	ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);
	bool bDatabaseUpdated = false;

	// Fill up the response.
	FConcertAdmin_GetSessionRepositoriesResponse ResponseData;
	for (FConcertServerSessionRepository& Repository : SessionRepositoryDb.Repositories)
	{
		if (Repository.bMounted && !FPlatformProcess::IsApplicationRunning(Repository.ProcessId)) // Check if the state still hold.
		{
			Repository.bMounted = false; // Update the state.
			Repository.ProcessId = 0;
			bDatabaseUpdated = true;
		}
		ResponseData.SessionRepositories.Add(FConcertSessionRepositoryInfo{Repository.RepositoryId, Repository.bMounted});
	}

	if (bDatabaseUpdated)
	{
		ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
	}

	return FConcertAdmin_GetSessionRepositoriesResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_DropSessionRepositoriesResponse> FConcertServer::HandleDropSessionRepositoriesRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_DropSessionRepositoriesRequest* Message = Context.GetMessage<FConcertAdmin_DropSessionRepositoriesRequest>();
	FConcertAdmin_DropSessionRepositoriesResponse ResponseData;

	// Drop the repository currently mounted by this process.
	for (const FGuid& RepositoryId : Message->RepositoryIds)
	{
		if (UnmountSessionRepository(RepositoryId, /*bDropped*/true))
		{
			ResponseData.DroppedRepositoryIds.Add(RepositoryId);
		}
	}

	// Drop the repository that aren't mounted, but found in the global repository database.
	{
		FSystemWideCriticalSection ScopedSystemWideMutex(ConcertServerUtil::GetServerSystemMutexName());
		FConcertServerSessionRepositoryDatabase SessionRepositoryDb;
		ConcertServerUtil::LoadSessionRepositoryDatabase(Role, SessionRepositoryDb);

		// Drop the repositories.
		for (const FGuid& RepositoryId : Message->RepositoryIds)
		{
			int32 Index = SessionRepositoryDb.Repositories.IndexOfByPredicate([&RepositoryId](const FConcertServerSessionRepository& Repository) { return Repository.RepositoryId == RepositoryId; });
			if (Index == INDEX_NONE)
			{
				ResponseData.DroppedRepositoryIds.Add(RepositoryId); // Not mapped in the DB -> successufully dropped.
				continue;
			}
			
			FConcertServerSessionRepository& Repository = SessionRepositoryDb.Repositories[Index];
			if (!Repository.bMounted || !FPlatformProcess::IsApplicationRunning(Repository.ProcessId)) // Not mounted or mounted by a dead process.
			{
				// Check if the server can delete the folder safely i.e. it has the standard structure managed by the server.
				if (!Repository.RepositoryRootDir.IsEmpty())
				{
					FString ReposDir = Repository.RepositoryRootDir / Repository.RepositoryId.ToString();
					ConcertUtil::DeleteDirectoryTree(*ReposDir);
				}

				// Unmap it.
				SessionRepositoryDb.Repositories.RemoveAt(Index);
				ResponseData.DroppedRepositoryIds.Add(RepositoryId);
			}
		}

		if (ResponseData.DroppedRepositoryIds.Num())
		{
			ConcertServerUtil::SaveSessionRepositoryDatabase(Role, SessionRepositoryDb);
		}
	}

	return FConcertAdmin_DropSessionRepositoriesResponse::AsFuture(MoveTemp(ResponseData));
}


EConcertResponseCode FConcertServer::ValidateFileSharing(const FConcertSessionFileShare& InRequest, FConcertSessionFileShare& OutFileShare, FText& OutFailureReason)
{
	// Assume that we can't support file sharing and prove otherwise.
	OutFileShare.bFileShareEnabled = false;
	OutFileShare.FileShareRoot = TEXT("");

	/* Supply any file share information back to the client. */
	if (FileSharingService && InRequest.bFileShareEnabled && Settings->bEnableFileSharing)
	{
		// Try to accept the file sharing challenge
		OutFileShare.Challenge = InRequest.Challenge;

		FString BaseDir = FPaths::ConvertRelativePathToFull(Settings->RootFileSharingPath.Path);
		FString Normalized = FPaths::ConvertRelativePathToFull(InRequest.Challenge.Path);

		if (!BaseDir.IsEmpty() && FPaths::IsUnderDirectory(Normalized, BaseDir))
		{
			TSharedPtr<FArchive> Reader = FileSharingService->CreateReader(Normalized);
			if (Reader)
			{
				FConcertSessionChallengeData ChallengeData;
				*Reader << ChallengeData.ChallengeKey;
				*Reader << ChallengeData.CreationTimeInTicks;
	
				if (ChallengeData == InRequest.Challenge.Data)
				{
					// We can accept the challenge.
					OutFileShare.bFileShareEnabled = true;
					OutFileShare.FileShareRoot = Settings->RootFileSharingPath.Path;
				}
			}	
		}

		if (!OutFileShare.bFileShareEnabled)
		{
			OutFailureReason = LOCTEXT("Error_FailedChallenge", "The client failed to prove they could support file sharing.");
			return EConcertResponseCode::Failed;
		}
	}
	else if (FileSharingService && FileSharingService->IsEnabled() && Settings->bEnableFileSharing)
	{
		OutFailureReason = LOCTEXT("Error_NoFileSharing", "The server is setup for file sharing. Clients joining a session must support file sharing.");
		return EConcertResponseCode::Failed;
	}
	else if (InRequest.bFileShareEnabled)
	{
		OutFailureReason = LOCTEXT("Error_FileSharingFromClient", "The client asked for file sharing but the server does not support it.");		
		return EConcertResponseCode::Failed;
	}

	return EConcertResponseCode::Success;
}

template<typename TCreateOrJoinMessage>
bool FConcertServer::FillSessionResponseInfoFromFileShareChallenge(const TCreateOrJoinMessage* Message, FConcertAdmin_SessionInfoResponse& ResponseData, const FConcertSessionInfo& SessionInfo)
{
	const TCHAR* ServerSessionNamePtr = *SessionInfo.SessionName;

	if (Message->FileShare.bFileShareEnabled)
	{
		UE_LOGF(LogConcert, Display, "Validating file sharing challenge from user %ls to join session %ls (Id: %ls, Owner: %ls)", *Message->OwnerClientInfo.UserName, ServerSessionNamePtr, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}
	
	ResponseData.ResponseCode = ValidateFileSharing(Message->FileShare, ResponseData.FileShare, ResponseData.Reason);

	if (ResponseData.ResponseCode != EConcertResponseCode::Success)
	{
		UE_LOGF(LogConcert, Display, "Refusing user %ls to join session %ls because of failed file share challenge. (Id: %ls, Owner: %ls, Reason: %ls)", *Message->OwnerClientInfo.UserName, ServerSessionNamePtr, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName, *ResponseData.Reason.ToString());
	}
	else
	{
		// Fill out the session info to return back to the client.
		ResponseData.SessionInfo = SessionInfo;
		UE_LOGF(LogConcert, Display, "Allowing user %ls to join session %ls.  File sharing is %ls. (Id: %ls, Owner: %ls)", *Message->OwnerClientInfo.UserName, ServerSessionNamePtr, ResponseData.FileShare.bFileShareEnabled ? TEXT("enabled") : TEXT("disabled"), *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}

	return ResponseData.ResponseCode == EConcertResponseCode::Success;  
}

TFuture<FConcertAdmin_SessionInfoResponse> FConcertServer::HandleCreateSessionRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_CreateSessionRequest* Message = Context.GetMessage<FConcertAdmin_CreateSessionRequest>();

	// We have a valid session if it succeeded
	FConcertAdmin_SessionInfoResponse ResponseData;

	// Create a new server session
	FText CreateFailureReason;
	TSharedPtr<IConcertServerSession> NewServerSession;
	{
		FConcertSessionInfo SessionInfo = CreateSessionInfo();
		SessionInfo.OwnerInstanceId = Message->OwnerClientInfo.InstanceInfo.InstanceId;
		SessionInfo.OwnerUserName = Message->OwnerClientInfo.UserName;
		SessionInfo.OwnerDeviceName = Message->OwnerClientInfo.DeviceName;
		SessionInfo.SessionName = Message->SessionName;
		SessionInfo.Settings = Message->SessionSettings;
		SessionInfo.VersionInfos.Add(Message->VersionInfo);
		if (FillSessionResponseInfoFromFileShareChallenge(Message, ResponseData, SessionInfo))
		{
			// Copy the session file sharing information to the user. 
			SessionInfo.FileSharing = ResponseData.FileShare;
			NewServerSession = CreateSession(SessionInfo, CreateFailureReason);
			if (NewServerSession)
			{
				ResponseData.SessionInfo = NewServerSession->GetSessionInfo();
			}
		}
	}

	if (!NewServerSession)
	{
		ResponseData.ResponseCode = EConcertResponseCode::Failed;
		ResponseData.Reason = CreateFailureReason;
		UE_LOGF(LogConcert, Display, "Session creation failed. (User: %ls, Reason: %ls)", *Message->OwnerClientInfo.UserName, *ResponseData.Reason.ToString());
	}

	return FConcertAdmin_SessionInfoResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_SessionInfoResponse> FConcertServer::HandleFindSessionRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_FindSessionRequest* Message = Context.GetMessage<FConcertAdmin_FindSessionRequest>();

	FConcertAdmin_SessionInfoResponse ResponseData;

	// Find the session requested
	TSharedPtr<IConcertServerSession> ServerSession = GetLiveSession(Message->SessionId);
	if (CanJoinSession(ServerSession, Message->SessionSettings, Message->VersionInfo, Message->ConcertEndpointId, Message->OwnerClientInfo, &ResponseData.Reason))
	{
		FillSessionResponseInfoFromFileShareChallenge(Message, ResponseData, ServerSession->GetSessionInfo());
	}
	else
	{
		const TCHAR* ServerSessionNamePtr = ServerSession ? *ServerSession->GetName() : TEXT("<unknown>");
		ResponseData.ResponseCode = EConcertResponseCode::Failed;
		UE_LOGF(LogConcert, Display, "Refusing user %ls to join session %ls (Id: %ls, Owner: %ls, Reason: %ls)", *Message->OwnerClientInfo.UserName, ServerSessionNamePtr, *Message->SessionId.ToString(), ServerSession ? *ServerSession->GetSessionInfo().OwnerUserName : TEXT("unknown owner"), *ResponseData.Reason.ToString());
	}

	return FConcertAdmin_SessionInfoResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_SessionInfoResponse> FConcertServer::HandleCopySessionRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_CopySessionRequest* Message = Context.GetMessage<FConcertAdmin_CopySessionRequest>();

	// Restore the server session
	FText FailureReason;
	TSharedPtr<IConcertServerSession> NewServerSession;
	{
		FConcertSessionInfo SessionInfo = CreateSessionInfo();
		SessionInfo.OwnerInstanceId = Message->OwnerClientInfo.InstanceInfo.InstanceId;
		SessionInfo.OwnerUserName = Message->OwnerClientInfo.UserName;
		SessionInfo.OwnerDeviceName = Message->OwnerClientInfo.DeviceName;
		SessionInfo.SessionName = Message->SessionName;
		SessionInfo.Settings = Message->SessionSettings;
		SessionInfo.VersionInfos.Add(Message->VersionInfo);
		NewServerSession = Message->bRestoreOnly ?
			RestoreSession(Message->SessionId, SessionInfo, Message->SessionFilter, FailureReason) :
			CopySession(Message->SessionId, SessionInfo, Message->SessionFilter, FailureReason);
	}

	// We have a valid session if it succeeded
	FConcertAdmin_SessionInfoResponse ResponseData;
	if (NewServerSession)
	{
		ResponseData.SessionInfo = NewServerSession->GetSessionInfo();
		ResponseData.ResponseCode = EConcertResponseCode::Success;
	}
	else
	{
		ResponseData.ResponseCode = EConcertResponseCode::Failed;
		ResponseData.Reason = FailureReason;
		UE_LOGF(LogConcert, Display, "Session copy failed. (User: %ls, Reason: %ls)", *Message->OwnerClientInfo.UserName, *ResponseData.Reason.ToString());
	}

	return FConcertAdmin_SessionInfoResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_ArchiveSessionResponse> FConcertServer::HandleArchiveSessionRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_ArchiveSessionRequest* Message = Context.GetMessage<FConcertAdmin_ArchiveSessionRequest>();

	FConcertAdmin_ArchiveSessionResponse ResponseData;

	// Find the session requested.
	TSharedPtr<IConcertServerSession> ServerSession = GetLiveSession(Message->SessionId);
	ResponseData.SessionId = Message->SessionId;
	ResponseData.SessionName = ServerSession ? ServerSession->GetName() : TEXT("<unknown>");
	if (ServerSession)
	{
		FText FailureReason;
		const FGuid ArchivedSessionId = ArchiveSession(Message->SessionId, Message->ArchiveNameOverride, Message->SessionFilter, FailureReason);
		if (ArchivedSessionId.IsValid())
		{
			const FConcertSessionInfo& ArchivedSessionInfo = ArchivedSessions.FindChecked(ArchivedSessionId);
			ResponseData.ResponseCode = EConcertResponseCode::Success;
			ResponseData.ArchiveId = ArchivedSessionId;
			ResponseData.ArchiveName = ArchivedSessionInfo.SessionName;
			UE_LOGF(LogConcert, Display, "User %ls archived session %ls (%ls) as %ls (%ls)", *Message->UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ResponseData.ArchiveName, *ResponseData.ArchiveId.ToString());
		}
		else
		{
			ResponseData.ResponseCode = EConcertResponseCode::Failed;
			ResponseData.Reason = FailureReason;
			UE_LOGF(LogConcert, Display, "User %ls failed to archive session %ls (Id: %ls, Reason: %ls)", *Message->UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ResponseData.Reason.ToString());
		}
	}
	else
	{
		ResponseData.ResponseCode = EConcertResponseCode::Failed;
		ResponseData.Reason = LOCTEXT("Error_SessionDoesNotExist", "Session does not exist.");
		UE_LOGF(LogConcert, Display, "User %ls failed to archive session %ls (Id: %ls, Reason: %ls)", *Message->UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ResponseData.Reason.ToString());
	}

	return FConcertAdmin_ArchiveSessionResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_RenameSessionResponse> FConcertServer::HandleRenameSessionRequest(const FConcertMessageContext& Context)
{
	return FConcertAdmin_RenameSessionResponse::AsFuture(RenameSessionInternal(*Context.GetMessage<FConcertAdmin_RenameSessionRequest>(), /*bCheckPermission*/true));
}

FConcertAdmin_RenameSessionResponse FConcertServer::RenameSessionInternal(const FConcertAdmin_RenameSessionRequest& Request, bool bCheckPermission)
{
	FConcertAdmin_RenameSessionResponse ResponseData;
	ResponseData.SessionId = Request.SessionId;
	ResponseData.ResponseCode = EConcertResponseCode::Failed;

	if (TSharedPtr<IConcertServerSession> ServerSession = GetLiveSession(Request.SessionId)) // Live session?
	{
		ResponseData.OldName = ServerSession->GetName();

		if (bCheckPermission && !IsRequestFromSessionOwner(ServerSession, Request.UserName, Request.DeviceName)) // Not owner?
		{
			ResponseData.Reason = LOCTEXT("Error_Rename_InvalidPerms_NotOwner", "Not the session owner.");
			UE_LOGF(LogConcert, Error, "User %ls failed to rename live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ServerSession->GetName(), *ResponseData.SessionId.ToString(), *ServerSession->GetSessionInfo().OwnerUserName, *ResponseData.Reason.ToString());
		}
		else if (GetLiveSessionIdByName(Request.NewName).IsValid()) // Name collision?
		{
			ResponseData.Reason = FText::Format(LOCTEXT("Error_Rename_SessionAlreadyExists", "Session '{0}' already exists"),  FText::AsCultureInvariant(Request.NewName));
			UE_LOGF(LogConcert, Error, "User %ls failed to rename live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ServerSession->GetName(), *ResponseData.SessionId.ToString(), *ServerSession->GetSessionInfo().OwnerUserName, *ResponseData.Reason.ToString());
		}
		else
		{
			ServerSession->SetName(Request.NewName);
			EventSink->OnLiveSessionRenamed(*this, ServerSession.ToSharedRef());

			ResponseData.ResponseCode = EConcertResponseCode::Success;
			UE_LOGF(LogConcert, Display, "User %ls renamed live session %ls from %ls to %ls", *Request.UserName, *ResponseData.SessionId.ToString(), *ResponseData.OldName, *ServerSession->GetName());
		}
	}
	else if (FConcertSessionInfo* ArchivedSessionInfo = ArchivedSessions.Find(Request.SessionId)) // Archive session?
	{
		ResponseData.OldName = ArchivedSessionInfo->SessionName;

		if (bCheckPermission && (ArchivedSessionInfo->OwnerUserName != Request.UserName || ArchivedSessionInfo->OwnerDeviceName != Request.DeviceName)) // Not the owner?
		{
			ResponseData.Reason = LOCTEXT("Error_Rename_InvalidPerms_NotOwner", "Not the session owner.");
			UE_LOGF(LogConcert, Display, "User %ls failed to rename archived session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ArchivedSessionInfo->SessionName, *ResponseData.SessionId.ToString(), *ArchivedSessionInfo->OwnerUserName, *ResponseData.Reason.ToString());
		}
		else if (GetArchivedSessionIdByName(Request.NewName).IsValid()) // Name collision?
		{
			ResponseData.Reason = FText::Format(LOCTEXT("Error_Rename_ArchiveAlreadyExists", "Archive '{0}' already exists"), FText::AsCultureInvariant(Request.NewName));
			UE_LOGF(LogConcert, Error, "User %ls failed to rename archived session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ArchivedSessionInfo->SessionName, *ResponseData.SessionId.ToString(), *ArchivedSessionInfo->OwnerUserName, *ResponseData.Reason.ToString());
		}
		else
		{
			ArchivedSessionInfo->SessionName = Request.NewName;
			EventSink->OnArchivedSessionRenamed(*this, GetSessionSavedDir(Request.SessionId), *ArchivedSessionInfo);

			ResponseData.ResponseCode = EConcertResponseCode::Success;
			UE_LOGF(LogConcert, Display, "User %ls renamed archived session %ls from %ls to %ls", *Request.UserName, *ResponseData.SessionId.ToString(), *ResponseData.OldName, *Request.NewName);
		}
	}
	else // Not found?
	{
		ResponseData.Reason = LOCTEXT("Error_Rename_DoesNotExist", "Session does not exist.");
		UE_LOGF(LogConcert, Display, "User %ls failed to rename session (Id: %ls, Reason: %ls)", *Request.UserName, *ResponseData.SessionId.ToString(), *ResponseData.Reason.ToString());
	}

	return ResponseData;
}

TFuture<FConcertAdmin_BatchDeleteSessionResponse> FConcertServer::HandleBatchDeleteSessionRequest(const FConcertMessageContext& Context)
{
	FConcertAdmin_BatchDeleteSessionResponse ResponseData;
	ResponseData.ResponseCode = EConcertResponseCode::Failed;
	
	const FConcertAdmin_BatchDeleteSessionRequest& Request = *Context.GetMessage<FConcertAdmin_BatchDeleteSessionRequest>();
	TMap<FGuid, FString> SessionNames;
	if (ValidateBatchDeletionRequest(Request, ResponseData, SessionNames))
	{
		ResponseData.ResponseCode = EConcertResponseCode::Success;
		FConcertAdmin_DeleteSessionRequest DeleteSingleSessionRequest;
		for (const FGuid& SessionToDelete : Request.SessionIds)
		{
			const bool bSkip = ResponseData.NotOwnedByClient.ContainsByPredicate([&SessionToDelete](const FDeletedSessionInfo& Info ){ return Info.SessionId == SessionToDelete; });
			if (bSkip)
			{
				continue;
			}
			
			DeleteSingleSessionRequest.SessionId = SessionToDelete;
			const FConcertAdmin_DeleteSessionResponse DeleteResponse = DeleteSessionInternal(DeleteSingleSessionRequest, false);
			if (DeleteResponse.ResponseCode == EConcertResponseCode::Success)
			{
				ResponseData.DeletedItems.Add({ SessionToDelete, SessionNames[SessionToDelete] });
			}
			else
			{
				// We may already have deleted some files ... maybe we should restore them in the future...
				ResponseData.ResponseCode = EConcertResponseCode::Failed;
				break;
			}
		}
	}

	return FConcertAdmin_BatchDeleteSessionResponse::AsFuture(MoveTemp(ResponseData));
}

bool FConcertServer::ValidateBatchDeletionRequest(const FConcertAdmin_BatchDeleteSessionRequest& Request, FConcertAdmin_BatchDeleteSessionResponse& OutResponse, TMap<FGuid, FString>& PreparedSessionInfo) const
{
	TArray<FDeletedSessionInfo> NotOwnedByClient;
	for (const FGuid& SessionToDelete : Request.SessionIds)
	{
		if (TSharedPtr<IConcertServerSession> ServerSession = GetLiveSession(SessionToDelete))
		{
			const bool bHasPermission = IsRequestFromSessionOwner(ServerSession, Request.UserName, Request.DeviceName);
			if (!bHasPermission && (Request.Flags & EBatchSessionDeletionFlags::SkipForbiddenSessions) != EBatchSessionDeletionFlags::Strict)
			{
				NotOwnedByClient.Add({ SessionToDelete, ServerSession->GetName() });
			}
			else if (!bHasPermission)
			{
				OutResponse.Reason = LOCTEXT("Error_BatchDelete_InvalidPerms_NotOwner", "Not the session owner.");
				UE_LOGF(LogConcert, Display, "User %ls failed to delete live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ServerSession->GetName(), *SessionToDelete.ToString(), *ServerSession->GetSessionInfo().OwnerUserName, *OutResponse.Reason.ToString());
				return false;
			}

			PreparedSessionInfo.Add(SessionToDelete, ServerSession->GetName());
		}
		else if (const FConcertSessionInfo* ArchivedSessionInfo = ArchivedSessions.Find(SessionToDelete))
		{
			const bool bHasPermission = IsRequestFromSessionOwner(*ArchivedSessionInfo, Request.UserName, Request.DeviceName);
			if (!bHasPermission && (Request.Flags & EBatchSessionDeletionFlags::SkipForbiddenSessions) != EBatchSessionDeletionFlags::Strict)
			{
				NotOwnedByClient.Add({ SessionToDelete, ArchivedSessionInfo->SessionName });
			}
			else if (!bHasPermission)
			{
				OutResponse.Reason = LOCTEXT("Error_BatchDelete_InvalidPerms_NotOwner", "Not the session owner.");
				UE_LOGF(LogConcert, Display, "User %ls failed to delete live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ArchivedSessionInfo->SessionName, *SessionToDelete.ToString(), *ArchivedSessionInfo->OwnerUserName, *OutResponse.Reason.ToString());
				return false;
			}
			
			PreparedSessionInfo.Add(SessionToDelete, ArchivedSessionInfo->SessionName);
		}
		else
		{
			OutResponse.Reason = FText::Format(LOCTEXT("Error_BatchDelete_SessionDoesNotExist", "Session ID {0} does not exist."), FText::FromString(SessionToDelete.ToString()));
			UE_LOGF(LogConcert, Display, "User %ls failed to delete session (Id: %ls, Reason: %ls)", *Request.UserName, *SessionToDelete.ToString(), *OutResponse.Reason.ToString());
			return false;
		}
	}

	OutResponse.NotOwnedByClient = MoveTemp(NotOwnedByClient);
	return true;
}

TFuture<FConcertAdmin_DeleteSessionResponse> FConcertServer::HandleDeleteSessionRequest(const FConcertMessageContext & Context)
{
	return FConcertAdmin_DeleteSessionResponse::AsFuture(DeleteSessionInternal(*Context.GetMessage<FConcertAdmin_DeleteSessionRequest>(), /*bCheckPermission*/true));
}

FConcertAdmin_DeleteSessionResponse FConcertServer::DeleteSessionInternal(const FConcertAdmin_DeleteSessionRequest& Request, bool bCheckPermission)
{
	FConcertAdmin_DeleteSessionResponse ResponseData;
	ResponseData.SessionId = Request.SessionId;
	ResponseData.ResponseCode = EConcertResponseCode::Failed;

	if (TSharedPtr<IConcertServerSession> ServerSession = GetLiveSession(Request.SessionId)) // Live session?
	{
		ResponseData.SessionName = ServerSession->GetName();

		if (bCheckPermission && !IsRequestFromSessionOwner(ServerSession, Request.UserName, Request.DeviceName))
		{
			ResponseData.Reason = LOCTEXT("Error_Delete_InvalidPerms_NotOwner", "Not the session owner.");
			UE_LOGF(LogConcert, Display, "User %ls failed to delete live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ServerSession->GetSessionInfo().OwnerUserName, *ResponseData.Reason.ToString());
		}
		else if (!DestroyLiveSession(Request.SessionId, /*bDeleteSessionData*/true))
		{
			ResponseData.Reason = LOCTEXT("Error_Delete_SessionFailedToDestroy", "Failed to destroy session.");
			UE_LOGF(LogConcert, Display, "User %ls failed to delete live session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ServerSession->GetSessionInfo().OwnerUserName, *ResponseData.Reason.ToString());
		}
		else // Succeeded to delete the session.
		{
			ResponseData.ResponseCode = EConcertResponseCode::Success;
			UE_LOGF(LogConcert, Display, "User %ls deleted live session %ls (%ls)", *Request.UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString());
		}
	}
	else if (const FConcertSessionInfo* ArchivedSessionInfo = ArchivedSessions.Find(Request.SessionId)) // Archived session?
	{
		ResponseData.SessionName = ArchivedSessionInfo->SessionName;

		if (bCheckPermission && (ArchivedSessionInfo->OwnerUserName != Request.UserName || ArchivedSessionInfo->OwnerDeviceName != Request.DeviceName)) // Not the owner?
		{
			ResponseData.Reason = LOCTEXT("Error_Delete_InvalidPerms_NotOwner", "Not the session owner.");
			UE_LOGF(LogConcert, Display, "User %ls failed to delete archived session '%ls' (Id: %ls, Owner: %ls, Reason: %ls)", *Request.UserName, *ArchivedSessionInfo->SessionName, *ResponseData.SessionId.ToString(), *ArchivedSessionInfo->OwnerUserName, *ResponseData.Reason.ToString());
		}
		else if (!DestroyArchivedSession(Request.SessionId, /*bDeleteSessionData*/true))
		{
			ResponseData.Reason = LOCTEXT("Error_Delete_SessionFailedToDestroy", "Failed to destroy session.");
			UE_LOGF(LogConcert, Display, "User %ls failed to delete archived session '%ls' (Id: %ls, Reason: %ls)", *Request.UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString(), *ResponseData.Reason.ToString());
		}
		else // Succeeded to delete the session.
		{
			ResponseData.ResponseCode = EConcertResponseCode::Success;
			UE_LOGF(LogConcert, Display, "User %ls deleted archived session %ls (%ls)", *Request.UserName, *ResponseData.SessionName, *ResponseData.SessionId.ToString());
		}
	}
	else // Not found?
	{
		ResponseData.Reason = LOCTEXT("Error_Delete_SessionDoesNotExist", "Session does not exist.");
		UE_LOGF(LogConcert, Display, "User %ls failed to delete session (Id: %ls, Reason: %ls)", *Request.UserName, *ResponseData.SessionId.ToString(), *ResponseData.Reason.ToString());
	}

	return ResponseData;
}

TFuture<FConcertAdmin_GetAllSessionsResponse> FConcertServer::HandleGetAllSessionsRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_GetAllSessionsRequest* Message = Context.GetMessage<FConcertAdmin_GetAllSessionsRequest>();

	FConcertAdmin_GetAllSessionsResponse ResponseData;
	ResponseData.LiveSessions = GetLiveSessionInfos();
	for (const auto& ArchivedSessionPair : ArchivedSessions)
	{
		ResponseData.ArchivedSessions.Add(ArchivedSessionPair.Value);
	}

	return FConcertAdmin_GetAllSessionsResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_GetSessionsResponse> FConcertServer::HandleGetLiveSessionsRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_GetLiveSessionsRequest* Message = Context.GetMessage<FConcertAdmin_GetLiveSessionsRequest>();

	FConcertAdmin_GetSessionsResponse ResponseData;
	ResponseData.Sessions = GetLiveSessionInfos();
	
	return FConcertAdmin_GetSessionsResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_GetSessionsResponse> FConcertServer::HandleGetArchivedSessionsRequest(const FConcertMessageContext& Context)
{
	FConcertAdmin_GetSessionsResponse ResponseData;

	ResponseData.ResponseCode = EConcertResponseCode::Success;
	for (const auto& ArchivedSessionPair : ArchivedSessions)
	{
		ResponseData.Sessions.Add(ArchivedSessionPair.Value);
	}

	return FConcertAdmin_GetSessionsResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_GetSessionClientsResponse> FConcertServer::HandleGetSessionClientsRequest(const FConcertMessageContext& Context)
{
	const FConcertAdmin_GetSessionClientsRequest* Message = Context.GetMessage<FConcertAdmin_GetSessionClientsRequest>();

	FConcertAdmin_GetSessionClientsResponse ResponseData;
	ResponseData.SessionClients = ConcertUtil::GetSessionClients(*this, Message->SessionId);
	
	return FConcertAdmin_GetSessionClientsResponse::AsFuture(MoveTemp(ResponseData));
}

TFuture<FConcertAdmin_GetSessionActivitiesResponse> FConcertServer::HandleGetSessionActivitiesRequest(const FConcertMessageContext& Context)
{
	FConcertAdmin_GetSessionActivitiesResponse ResponseData;

	const FConcertAdmin_GetSessionActivitiesRequest* Message = Context.GetMessage<FConcertAdmin_GetSessionActivitiesRequest>();
	if (EventSink->GetUnmutedSessionActivities(*this, Message->SessionId, Message->FromActivityId, Message->ActivityCount, ResponseData.Activities, ResponseData.EndpointClientInfoMap, Message->bIncludeDetails))
	{
		ResponseData.ResponseCode = EConcertResponseCode::Success;
	}
	else // The only reason to get here is when the session is not found.
	{
		ResponseData.ResponseCode = EConcertResponseCode::Failed;
		ResponseData.Reason = LOCTEXT("Error_SessionActivities_SessionDoesNotExist", "Session does not exist or its database is corrupted.");
		UE_LOGF(LogConcert, Display, "Failed to fetch activities from session (Id: %ls, Reason: %ls)", *Message->SessionId.ToString(), *ResponseData.Reason.ToString());
	}

	return FConcertAdmin_GetSessionActivitiesResponse::AsFuture(MoveTemp(ResponseData));
}

bool FConcertServer::CanJoinSession(const TSharedPtr<IConcertServerSession>& ServerSession, const FConcertSessionSettings& SessionSettings, const FConcertSessionVersionInfo& SessionVersionInfo, const FGuid& EndpointId, const FConcertClientInfo& ClientInfo, FText* OutFailureReason)
{
	if (!ServerSession)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = LOCTEXT("Error_CanJoinSession_UnknownSession", "Unknown session");
		}
		return false;
	}

	if (OnConcertParticipantCanJoinSessionDelegate.IsBound())
	{
		if (!OnConcertParticipantCanJoinSessionDelegate.Execute(ServerSession->GetId(), EndpointId, ClientInfo, OutFailureReason))
		{
			return false;
		}
	}

	if (Settings->ServerSettings.bIgnoreSessionSettingsRestriction)
	{
		return true;
	}

	if (!ServerSession->GetSessionInfo().Settings.ValidateRequirements(SessionSettings, OutFailureReason))
	{
		return false;
	}

	if (ServerSession->GetSessionInfo().VersionInfos.Num() > 0 && !ServerSession->GetSessionInfo().VersionInfos.Last().Validate(SessionVersionInfo, EConcertVersionValidationMode::PatchCompatible, OutFailureReason))
	{
		return false;
	}

	return true;
}

bool FConcertServer::IsRequestFromSessionOwner(const TSharedPtr<IConcertServerSession>& SessionToDelete, const FString& FromUserName, const FString& FromDeviceName) const
{
	if (SessionToDelete)
	{
		const FConcertSessionInfo& SessionInfo = SessionToDelete->GetSessionInfo();
		return IsRequestFromSessionOwner(SessionInfo, FromUserName, FromDeviceName);
	}
	return false;
}

bool FConcertServer::IsRequestFromSessionOwner(const FConcertSessionInfo& SessionInfo, const FString& FromUserName, const FString& FromDeviceName) const
{
	return SessionInfo.OwnerUserName == FromUserName && SessionInfo.OwnerDeviceName == FromDeviceName;
}

TSharedPtr<IConcertServerSession> FConcertServer::CreateLiveSession(const FConcertSessionInfo& SessionInfo, const FConcertServerSessionRepository& InRepository)
{
	check(SessionInfo.SessionId.IsValid() && !SessionInfo.SessionName.IsEmpty());
	check(!LiveSessions.Contains(SessionInfo.SessionId) && !GetLiveSessionIdByName(SessionInfo.SessionName).IsValid());

	// Strip version info when using -CONCERTIGNORE
	FConcertSessionInfo LiveSessionInfo = SessionInfo;
	if (Settings->ServerSettings.bIgnoreSessionSettingsRestriction)
	{
		UE_CLOGF(LiveSessionInfo.VersionInfos.Num() > 0, LogConcert, Warning, "Clearing version information when creating session '%ls' due to -CONCERTIGNORE. This session will be unversioned!", *LiveSessionInfo.SessionName);
		LiveSessionInfo.VersionInfos.Reset();
	}
	
	TSharedPtr<IConcertLocalEndpoint> SessionEndpoint = EndpointProvider->CreateLocalEndpoint(LiveSessionInfo.SessionName, Settings->EndpointSettings, [this](const FConcertEndpointContext& Context)
		{
			return ConcertUtil::CreateLogger(Context, [this](const FConcertLog& Log)
			{
				ConcertTransportEvents::OnConcertServerLogEvent().Broadcast(*this, Log);
			});
		});
	SessionEndpoint->OnConcertMessageAcknowledgementReceived().AddLambda(
		[this](const FConcertEndpointContext& LocalEndpoint, const FConcertEndpointContext& RemoteEndpoint, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& MessageContext)
		{
			OnConcertMessageAcknowledgementReceivedFromLocalEndpoint.Broadcast(LocalEndpoint, RemoteEndpoint, AckedMessage, MessageContext);
		});

	TSharedPtr<FConcertServerSession> LiveSession = MakeShared<FConcertServerSession>(
		LiveSessionInfo,
		Settings->ServerSettings,
		SessionEndpoint,
		InRepository.GetSessionWorkingDir(LiveSessionInfo.SessionId)
		);

	FInternalLiveSessionCreationParams CreationParams;
	CreationParams.OnModifiedCallback.BindSP(LiveSession.Get(), &FConcertServerSession::SetLastModifiedToNow);
	
	if (EventSink->OnLiveSessionCreated(*this, LiveSession.ToSharedRef(), CreationParams)) // EventSync could complete the session initialization?
	{
		LiveSessions.Add(LiveSessionInfo.SessionId, LiveSession);
		LiveSession->Startup();
		OnConcertServerSessionStartupDelegate.Broadcast(LiveSession);
		return LiveSession;
	}

	return nullptr;
}

bool FConcertServer::DestroyLiveSession(const FGuid& LiveSessionId, const bool bDeleteSessionData)
{
	TSharedPtr<IConcertServerSession> LiveSession = LiveSessions.FindRef(LiveSessionId);
	if (LiveSession)
	{
		EventSink->OnLiveSessionDestroyed(*this, LiveSession.ToSharedRef());
		LiveSession->Shutdown();
		LiveSessions.Remove(LiveSessionId);

		if (bDeleteSessionData)
		{
			ConcertUtil::DeleteDirectoryTree(*GetSessionWorkingDir(LiveSessionId));
		}

		return true;
	}

	return false;
}

FGuid FConcertServer::ArchiveLiveSession(const FGuid& LiveSessionId, const FString& ArchivedSessionNameOverride, const FConcertSessionFilter& SessionFilter, FGuid ArchiveSessionIdOverride)
{
	TSharedPtr<IConcertServerSession> LiveSession = LiveSessions.FindRef(LiveSessionId);
	if (LiveSession)
	{
		FString ArchivedSessionName = ArchivedSessionNameOverride;
		if (ArchivedSessionName.IsEmpty())
		{
			ArchivedSessionName = ConcertServerUtil::GetArchiveName(*LiveSession->GetName(), LiveSession->GetSessionInfo().Settings);
		}
		{
			const FGuid ArchivedSessionId = GetArchivedSessionIdByName(ArchivedSessionName);
			DestroyArchivedSession(ArchivedSessionId, /*bDeleteSessionData*/true);
		}

		// Find the live session repository to stored the archive in the same one.
		const FConcertServerSessionRepository& SessionRepository = GetSessionRepository(LiveSession->GetSessionInfo().SessionId);

		FConcertSessionInfo ArchivedSessionInfo = LiveSession->GetSessionInfo();
		ArchivedSessionInfo.SessionId = ensure(ArchiveSessionIdOverride.IsValid()) ? MoveTemp(ArchiveSessionIdOverride) : FGuid::NewGuid();
		ArchivedSessionInfo.SessionName = MoveTemp(ArchivedSessionName);
		if (EventSink->ArchiveSession(*this, LiveSession.ToSharedRef(), SessionRepository.GetSessionSavedDir(ArchivedSessionInfo.SessionId), ArchivedSessionInfo, SessionFilter))
		{
			UE_LOGF(LogConcert, Display, "Live session '%ls' (%ls) was archived as '%ls' (%ls)", *LiveSession->GetName(), *LiveSession->GetId().ToString(), *ArchivedSessionInfo.SessionName, *ArchivedSessionInfo.SessionId.ToString());
			if (CreateArchivedSession(ArchivedSessionInfo))
			{
				return ArchivedSessionInfo.SessionId;
			}
		}
	}

	return FGuid();
}

bool FConcertServer::CreateArchivedSession(const FConcertSessionInfo& SessionInfo)
{
	check(SessionInfo.SessionId.IsValid() && !SessionInfo.SessionName.IsEmpty());
	check(!ArchivedSessions.Contains(SessionInfo.SessionId) && !GetArchivedSessionIdByName(SessionInfo.SessionName).IsValid());

	ArchivedSessions.Add(SessionInfo.SessionId, SessionInfo);
	return EventSink->OnArchivedSessionCreated(*this, GetSessionSavedDir(SessionInfo.SessionId), SessionInfo);
}

bool FConcertServer::DestroyArchivedSession(const FGuid& ArchivedSessionId, const bool bDeleteSessionData)
{
	if (ArchivedSessions.Contains(ArchivedSessionId))
	{
		EventSink->OnArchivedSessionDestroyed(*this, ArchivedSessionId);
		ArchivedSessions.Remove(ArchivedSessionId);

		if (bDeleteSessionData)
		{
			ConcertUtil::DeleteDirectoryTree(*GetSessionSavedDir(ArchivedSessionId));
		}

		return true;
	}

	return false;
}

TSharedPtr<IConcertServerSession> FConcertServer::RestoreArchivedSession(const FGuid& ArchivedSessionId, const FConcertSessionInfo& NewSessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason)
{
	check(NewSessionInfo.SessionId.IsValid());

	if (const FConcertSessionInfo* ArchivedSessionInfo = ArchivedSessions.Find(ArchivedSessionId))
	{
		// Find the archived session repository to restore the session in the same one.
		const FConcertServerSessionRepository& ArchivedSessionRepository = GetSessionRepository(ArchivedSessionId);

		FString LiveSessionName = NewSessionInfo.SessionName;
		if (LiveSessionName.IsEmpty())
		{
			LiveSessionName = ArchivedSessionInfo->SessionName;
		}
		{
			const FGuid LiveSessionId = GetLiveSessionIdByName(LiveSessionName);
			DestroyLiveSession(LiveSessionId, /*bDeleteSessionData*/true);
		}

		FConcertSessionInfo LiveSessionInfo = NewSessionInfo;
		// Detect restoring the same archived session twice by hashing archived ID
		LiveSessionInfo.SessionId = FGuid::NewGuid();
		LiveSessionInfo.SessionName = MoveTemp(LiveSessionName);
		LiveSessionInfo.VersionInfos = ArchivedSessionInfo->VersionInfos;
		LiveSessionInfo.SetLastModifiedToNow();

		// Ensure the new version is compatible with the old version, and append this new version if it is different to the last used version
		// Note: Older archived sessions didn't used to have any version info stored for them, and the version info may be missing completely when using -CONCERTIGNORE
		if (Settings->ServerSettings.bIgnoreSessionSettingsRestriction)
		{
			UE_CLOGF(LiveSessionInfo.VersionInfos.Num() > 0, LogConcert, Warning, "Clearing version information when restoring session '%ls' due to -CONCERTIGNORE. This may lead to instability and crashes!", *NewSessionInfo.SessionName);
			LiveSessionInfo.VersionInfos.Reset();
		}
		else if (NewSessionInfo.VersionInfos.Num() > 0)
		{
			check(NewSessionInfo.VersionInfos.Num() == 1);
			const FConcertSessionVersionInfo& NewVersionInfo = NewSessionInfo.VersionInfos[0];

			if (LiveSessionInfo.VersionInfos.Num() > 0)
			{
				if (!LiveSessionInfo.VersionInfos.Last().Validate(NewVersionInfo, EConcertVersionValidationMode::Compatible, &OutFailureReason))
				{
					UE_LOGF(LogConcert, Error, "An attempt to restore session '%ls' was rejected due to a versioning incompatibility: %ls", *NewSessionInfo.SessionName, *OutFailureReason.ToString());
					return nullptr;
				}

				if (!LiveSessionInfo.VersionInfos.Last().Validate(NewVersionInfo, EConcertVersionValidationMode::Identical))
				{
					LiveSessionInfo.VersionInfos.Add(NewVersionInfo);
				}
			}
			else
			{
				LiveSessionInfo.VersionInfos.Add(NewVersionInfo);
			}
		}

		// Restore the session in the default repository (where new sessions should be created), unless it is unset.
		const FConcertServerSessionRepository& RestoredSessionRepository = DefaultSessionRepository.IsSet() ? DefaultSessionRepository.GetValue() : ArchivedSessionRepository;
		if (EventSink->RestoreSession(*this, ArchivedSessionId, RestoredSessionRepository.GetSessionWorkingDir(LiveSessionInfo.SessionId), LiveSessionInfo, SessionFilter))
		{
			UE_LOGF(LogConcert, Display, "Archived session '%ls' (%ls) was restored as '%ls' (%ls)", *ArchivedSessionInfo->SessionName, *ArchivedSessionInfo->SessionId.ToString(), *LiveSessionInfo.SessionName, *LiveSessionInfo.SessionId.ToString());
			return CreateLiveSession(LiveSessionInfo, RestoredSessionRepository);
		}
	}

	OutFailureReason = LOCTEXT("Error_RestoreSession_FailedToCopy", "Could not copy session data from the archive");
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
