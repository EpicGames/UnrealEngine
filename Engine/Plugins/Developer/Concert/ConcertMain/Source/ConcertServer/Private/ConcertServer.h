// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertServer.h"
#include "IConcertTransportModule.h"
#include "ConcertSettings.h"
#include "ConcertServerSessionRepositories.h"
#include "UObject/StrongObjectPtr.h"

class FConcertServerSession;
class IConcertServerEventSink;

/** Implements Concert interface */
class FConcertServer : public IConcertServer
{
public: 
	FConcertServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter, IConcertServerEventSink* InEventSink, const TSharedPtr<IConcertEndpointProvider>& InEndpointProvider);
	virtual ~FConcertServer();

	virtual const FString& GetRole() const override;

	virtual void Configure(const UConcertServerConfig* InSettings) override;
	virtual bool IsConfigured() const override;
	virtual const UConcertServerConfig* GetConfiguration() const override;
	virtual const FConcertServerInfo& GetServerInfo() const override;

	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) override;
	virtual TSharedPtr<IConcertFileSharingService> GetFileSharingService() const override;		

	virtual TArray<FConcertEndpointContext> GetRemoteAdminEndpoints() const override;
	virtual FOnConcertRemoteEndpointConnectionChanged& OnRemoteEndpointConnectionChanged() override;
	virtual FMessageAddress GetRemoteAddress(const FGuid& AdminEndpointId) const override;
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() override;

	virtual bool IsStarted() const override;
	virtual void Startup() override;
	virtual void Shutdown() override;

	virtual FGuid GetLiveSessionIdByName(const FString& InName) const override;
	virtual FGuid GetArchivedSessionIdByName(const FString& InName) const override;

	virtual FConcertSessionInfo CreateSessionInfo() const override;
	virtual TArray<FConcertSessionInfo> GetLiveSessionInfos() const override;
	virtual TArray<FConcertSessionInfo> GetArchivedSessionInfos() const override;
	virtual TArray<TSharedPtr<IConcertServerSession>> GetLiveSessions() const override;
	virtual TSharedPtr<IConcertServerSession> GetLiveSession(const FGuid& SessionId) const override;
	virtual TOptional<FConcertSessionInfo> GetArchivedSessionInfo(const FGuid& SessionId) const override;
	virtual TSharedPtr<IConcertServerSession> CreateSession(const FConcertSessionInfo& SessionInfo, FText& OutFailureReason) override;
	virtual TSharedPtr<IConcertServerSession> RestoreSession(const FGuid& SessionId, const FConcertSessionInfo& SessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason) override;
	virtual TSharedPtr<IConcertServerSession> CopySession(const FGuid& SrcSessionId, const FConcertSessionInfo& NewSessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason) override;
	virtual FGuid ArchiveSession(const FGuid& SessionId, const FString& ArchiveNameOverride, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason, FGuid ArchiveSessionIdOverride = FGuid::NewGuid()) override;
	virtual bool ExportSession(const FGuid& SessionId, const FConcertSessionFilter& SessionFilter, const FString& DestDir, bool bAnonymizeData, FText& OutFailureReason) override;
	virtual bool RenameSession(const FGuid& SessionId, const FString& NewName, FText& OutFailureReason) override;
	virtual bool DestroySession(const FGuid& SessionId, FText& OutFailureReason) override;
	virtual FOnConcertServerSessionStartup& OnConcertServerSessionStartup() override;
	virtual FOnConcertServerStartup& OnConcertServerStartup() override;
	virtual FOnConcertParticipantCanJoinSession& OnConcertParticipantCanJoinSession() override;

protected:

	FOnConcertServerSessionStartup OnConcertServerSessionStartupDelegate;
	FOnConcertServerStartup OnConcertServerStartupDelegate;
	FOnConcertParticipantCanJoinSession OnConcertParticipantCanJoinSessionDelegate;

private:
	/** Returns the root dir where the servers keeps the its internally created repositories (When the caller doesn't provide the paths). */
	const FString& GetSessionRepositoriesRootDir() const;

	const FConcertServerSessionRepository& GetSessionRepository(const FGuid& SessionId) const;
	FString GetSessionSavedDir(const FGuid& SessionId) const;
	FString GetSessionWorkingDir(const FGuid& SessionId) const;

	EConcertSessionRepositoryMountResponseCode MountSessionRepository(FConcertServerSessionRepository Repository, bool bCreateIfNotExist, bool bCleanWorkingDir, bool bCleanExpiredSessions, bool bSearchByPaths, bool bAsDefault);
	bool UnmountSessionRepository(const FGuid& RepositoryId, bool bDropped);
	bool MountDefaultSessionRepository(const UConcertServerConfig* ServerConfig);

	/**  */
	void HandleDiscoverServersEvent(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_MountSessionRepositoryResponse> HandleMountSessionRepositoryRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_GetSessionRepositoriesResponse> HandleGetSessionRepositoriesRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_DropSessionRepositoriesResponse> HandleDropSessionRepositoriesRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_SessionInfoResponse> HandleCreateSessionRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_SessionInfoResponse> HandleFindSessionRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_SessionInfoResponse> HandleCopySessionRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_ArchiveSessionResponse> HandleArchiveSessionRequest(const FConcertMessageContext& Context);

	/**  */
	TFuture<FConcertAdmin_RenameSessionResponse> HandleRenameSessionRequest(const FConcertMessageContext& Context);

	/**  */
	FConcertAdmin_RenameSessionResponse RenameSessionInternal(const FConcertAdmin_RenameSessionRequest& Request, bool bCheckPermission);

	/**  */
	TFuture<FConcertAdmin_BatchDeleteSessionResponse> HandleBatchDeleteSessionRequest(const FConcertMessageContext& Context);

	/** Checks whether the request is valid to serve. Writes any skipped sessions (if any) and error reasons into OutResponse. */
	bool ValidateBatchDeletionRequest(const FConcertAdmin_BatchDeleteSessionRequest& Request, FConcertAdmin_BatchDeleteSessionResponse& OutResponse, TMap<FGuid, FString>& OutPreparedSessionNames) const;
	
	/**  */
	TFuture<FConcertAdmin_DeleteSessionResponse> HandleDeleteSessionRequest(const FConcertMessageContext& Context);

	/**  */
	FConcertAdmin_DeleteSessionResponse DeleteSessionInternal(const FConcertAdmin_DeleteSessionRequest& Request, bool bCheckPermission);

	/**  */
	TFuture<FConcertAdmin_GetAllSessionsResponse> HandleGetAllSessionsRequest(const FConcertMessageContext& Context);

	/** */
	TFuture<FConcertAdmin_GetSessionsResponse> HandleGetLiveSessionsRequest(const FConcertMessageContext& Context);

	/** */
	TFuture<FConcertAdmin_GetSessionsResponse> HandleGetArchivedSessionsRequest(const FConcertMessageContext& Context);

	/** */
	TFuture<FConcertAdmin_GetSessionClientsResponse> HandleGetSessionClientsRequest(const FConcertMessageContext& Context);

	/** */
	TFuture<FConcertAdmin_GetSessionActivitiesResponse> HandleGetSessionActivitiesRequest(const FConcertMessageContext& Context);

	/** Recover the sessions found in the working directory into live session, build the list of archived sessions and rotate them, keeping only the N most recent. */
	void RecoverSessions(const FConcertServerSessionRepository& InRepository, bool bCleanupExpiredSessions);
	
	/** Sets each FConcertSessionInfo::LastModified time to the corresponding index in SessionCreationTimes. */
	void UpdateLastModified(TArray<FConcertSessionInfo>& SessionInfos, const TArray<FDateTime>& SessionCreationTimes);

	/**
	 * Migrate the live sessions from the working directory (before sessions being recovered into live one) to the archived directory.
	 * Expected to happen at start up, before RecoverSessions(), if UConcertServerConfig::bAutoArchiveOnReboot is true.
	 */
	void ArchiveOfflineSessions(const FConcertServerSessionRepository& InRepository);

	/** Check whether a connecting participant can join the session. */
	bool CanJoinSession(const TSharedPtr<IConcertServerSession>& ServerSession, const FConcertSessionSettings& SessionSettings, const FConcertSessionVersionInfo& SessionVersionInfo, const FGuid& EndpointId, const FConcertClientInfo& ClientInfo, FText* OutFailureReason = nullptr);

	/**  Validate that the request to delete or modify a session comes from the owner of that session. */
	bool IsRequestFromSessionOwner(const TSharedPtr<IConcertServerSession>& Session, const FString& FromUserName, const FString& FromDeviceName) const;
	/**  Validate that the request to delete or modify a session comes from the owner of that session. */
	bool IsRequestFromSessionOwner(const FConcertSessionInfo& SessionInfo, const FString& FromUserName, const FString& FromDeviceName) const;
	
	/**  */
	TSharedPtr<IConcertServerSession> CreateLiveSession(const FConcertSessionInfo& SessionInfo, const FConcertServerSessionRepository& InRepository);

	/**  */
	bool DestroyLiveSession(const FGuid& LiveSessionId, const bool bDeleteSessionData);

	/**  */
	FGuid ArchiveLiveSession(const FGuid& LiveSessionId, const FString& ArchivedSessionNameOverride, const FConcertSessionFilter& SessionFilter, FGuid ArchiveSessionIdOverride = FGuid::NewGuid());

	/**  */
	bool CreateArchivedSession(const FConcertSessionInfo& SessionInfo);

	/**  */
	bool DestroyArchivedSession(const FGuid& ArchivedSessionId, const bool bDeleteSessionData);

	/**  */
	TSharedPtr<IConcertServerSession> RestoreArchivedSession(const FGuid& ArchivedSessionId, const FConcertSessionInfo& NewSessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason);

	/** Validate a file a session join request for file sharing support. */
	EConcertResponseCode ValidateFileSharing(const FConcertSessionFileShare& InRequest, FConcertSessionFileShare& OutFileShare, FText& OutFailureReason);

	/** Helper template method that handles the two different message types that are used for joining a session (create / join existing). */
	template<typename TCreateOrJoinMessage>
	bool FillSessionResponseInfoFromFileShareChallenge(const TCreateOrJoinMessage* Message, FConcertAdmin_SessionInfoResponse& ResponseData, const FConcertSessionInfo& SessionInfo);
		
	/** The role of this server (eg, MultiUser, DisasterRecovery, etc) */
	FString Role;

	/** All session repositories used by this server. */
	TArray<FConcertServerSessionRepository> MountedSessionRepositories;

	/** The default directory(ies) scanned to reload existing sessions or used to store newly created ones. */
	TOptional<FConcertServerSessionRepository> DefaultSessionRepository;

	/** The status of the default session repository (essentially to know why it not mounted). */
	FText DefaultSessionRepositoryStatus;

	/** The session filter to apply when auto-archiving sessions */
	FConcertSessionFilter AutoArchiveSessionFilter;

	/** Sink functions for events that this server can emit */
	IConcertServerEventSink* EventSink;

	/** Factory for creating Endpoint */
	TSharedPtr<IConcertEndpointProvider> EndpointProvider;
	
	/** Administration endpoint for the server (i.e. creating, joining sessions) */
	TSharedPtr<IConcertLocalEndpoint> ServerAdminEndpoint;

	/** File sharing service that we use to verify reachability to client proof files. */
	TSharedPtr<IConcertFileSharingService> FileSharingService;
		
	/** Called when ServerAdminEndpoint emits IConcertLocalEndpoint::OnConcertRemoteEndpointConnectionChanged */
	FOnConcertRemoteEndpointConnectionChanged OnConcertRemoteEndpointConnectionChangedDelegate; 
	
	/** Server and Instance Info */
	FConcertServerInfo ServerInfo;

	/** Map of Live Sessions */
	TMap<FGuid, TSharedPtr<FConcertServerSession>> LiveSessions;

	/** Map of Archived Sessions */
	TMap<FGuid, FConcertSessionInfo> ArchivedSessions;

	/** Server settings object we were configured with */
	TStrongObjectPtr<const UConcertServerConfig> Settings;

	/** The root directory containing the session repositories. */
	FString SessionRepositoryRootDir;

	FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint OnConcertMessageAcknowledgementReceivedFromLocalEndpoint;
};
