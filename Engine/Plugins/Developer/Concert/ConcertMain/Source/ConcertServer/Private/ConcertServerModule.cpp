// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertServerModule.h"
#include "IConcertTransportModule.h"

#include "ConcertServer.h"

/**
 * Implements the Concert Client module
 */
class FConcertServerModule : public IConcertServerModule
{
public:

	virtual void StartupModule() override
	{
		EndpointProvider = IConcertTransportModule::Get().CreateEndpointProvider();
	}

	virtual void ShutdownModule() override
	{
		EndpointProvider.Reset();
	}

	virtual IConcertServerRef CreateServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter, IConcertServerEventSink* InEventSink) override
	{
		return MakeShared<FConcertServer, ESPMode::ThreadSafe>(InRole, InAutoArchiveSessionFilter, InEventSink, EndpointProvider);
	}

private:
	TSharedPtr<IConcertEndpointProvider> EndpointProvider;
};

IMPLEMENT_MODULE(FConcertServerModule, ConcertServer)
