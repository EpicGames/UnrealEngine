// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerSession.h"
#include "ConcertLogGlobal.h"
#include "ConcertMessages.h"
#include "IConcertEndpoint.h"
#include "Scratchpad/ConcertScratchpad.h"
#include "ConcertServerSettings.h"

#include "Containers/Ticker.h"
#include "Stats/Stats.h"
#include "UObject/StructOnScope.h"

const FName ConcertServerMessageIdName("ConcertMessageId");

FConcertServerSession::FConcertServerSession(const FConcertSessionInfo& InSessionInfo, const FConcertServerSettings& InSettings, TSharedPtr<IConcertLocalEndpoint> InServerSessionEndpoint, const FString& InSessionDirectory)
	: FConcertSessionCommonImpl(InSessionInfo)
	, ServerSessionEndpoint(MoveTemp(InServerSessionEndpoint))
	, SessionTickFrequency(0, 0, InSettings.SessionTickFrequencySeconds)
	, SessionDirectory(InSessionDirectory)
{
	// Make sure the session has the correct server endpoint ID set
	SessionInfo.ServerEndpointId = ServerSessionEndpoint->GetEndpointContext().EndpointId;
}

FConcertServerSession::~FConcertServerSession()
{
	// if the SessionTick is valid, Shutdown wasn't called
	check(!SessionTick.IsValid());
}

void FConcertServerSession::Startup()
{
	if (!SessionTick.IsValid())
	{
		CommonStartup();

		// Register to connection changed event
		ServerSessionEndpoint->OnRemoteEndpointConnectionChanged().AddRaw(this, &FConcertServerSession::HandleRemoteConnectionChanged);

		// Setup the session handlers
		ServerSessionEndpoint->SubscribeEventHandler<FConcertSession_DiscoverAndJoinSessionEvent>(this, &FConcertServerSession::HandleDiscoverAndJoinSessionEvent);
		ServerSessionEndpoint->RegisterEventHandler<FConcertSession_LeaveSessionEvent>(this, &FConcertServerSession::HandleLeaveSessionEvent);
		ServerSessionEndpoint->RegisterEventHandler<FConcertSession_UpdateClientInfoEvent>(this, &FConcertServerSession::HandleUpdateClientInfoEvent);

		// Setup Handlers for custom session messages
		ServerSessionEndpoint->RegisterEventHandler<FConcertSession_CustomEvent>(this, &FConcertServerSession::HandleCustomEvent);
		ServerSessionEndpoint->RegisterRequestHandler<FConcertSession_CustomRequest, FConcertSession_CustomResponse>(this, &FConcertServerSession::HandleCustomRequest);

		// Setup the session tick
		SessionTick = FTSTicker::GetCoreTicker().AddTicker(TEXT("ServerSession"), 0, [this](float DeltaSeconds)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FConcertServerSession_Tick);
			TickConnections(DeltaSeconds);
			return true;
		});

		UE_LOGF(LogConcert, Display, "Initialized Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}
}

void FConcertServerSession::Shutdown()
{
	if (SessionTick.IsValid())
	{
		// Unregister connection changed
		ServerSessionEndpoint->OnRemoteEndpointConnectionChanged().RemoveAll(this);

		// Unregister the session handlers
		ServerSessionEndpoint->UnsubscribeEventHandler<FConcertSession_DiscoverAndJoinSessionEvent>();
		ServerSessionEndpoint->UnregisterEventHandler<FConcertSession_LeaveSessionEvent>();
		ServerSessionEndpoint->UnregisterEventHandler<FConcertSession_UpdateClientInfoEvent>();

		// Unregister handlers for the custom session messages
		ServerSessionEndpoint->UnregisterEventHandler<FConcertSession_CustomEvent>();
		ServerSessionEndpoint->UnregisterRequestHandler<FConcertSession_CustomRequest>();

		// Unregister the session tick
		FTSTicker::RemoveTicker(SessionTick);
		SessionTick.Reset();

		CommonShutdown();

		UE_LOGF(LogConcert, Display, "Shutdown Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);
	}
}

FOnConcertServerSessionTick& FConcertServerSession::OnTick()
{
	return OnTickDelegate;
}

FOnConcertServerSessionClientChanged& FConcertServerSession::OnSessionClientChanged()
{
	return OnSessionClientChangedDelegate;
}

FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& FConcertServerSession::OnConcertMessageAcknowledgementReceived()
{
	return ServerSessionEndpoint->OnConcertMessageAcknowledgementReceived();
}

FString FConcertServerSession::GetSessionWorkingDirectory() const 
{
	return SessionDirectory;
}

void FConcertServerSession::InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InUnused)
{
	ensureMsgf(!InUnused, TEXT("Custom event sequencing is not supported on the Multi-user server. Events should arrive in sequence order."));

	if (DestinationEndpointIds.Num() == 0)
	{
		return;
	}

	// Build the event
	FConcertSession_CustomEvent CustomEvent;
	CommonBuildCustomEvent(EventType, EventData, SessionInfo.ServerEndpointId, DestinationEndpointIds, CustomEvent);

	// TODO: Optimize this so we can queue the event for multiple client endpoints at the same time
	TMap<FName, FString> Annotations;
	if (EnumHasAnyFlags(Flags, EConcertMessageFlags::UniqueId))
	{
		Annotations.Add(ConcertServerMessageIdName, FGuid::NewGuid().ToString());
	}
	for (const FGuid& DestinationEndpointId : DestinationEndpointIds)
	{
		// Send the event
		ServerSessionEndpoint->SendEvent(CustomEvent, DestinationEndpointId, Flags, Annotations);
	}
}

void FConcertServerSession::InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler)
{
	// Build the request
	FConcertSession_CustomRequest CustomRequest;
	CommonBuildCustomRequest(RequestType, RequestData, SessionInfo.ServerEndpointId, DestinationEndpointId, CustomRequest);

	// Send the request
	ServerSessionEndpoint->SendRequest<FConcertSession_CustomRequest, FConcertSession_CustomResponse>(CustomRequest, DestinationEndpointId)
		.Next([Handler](const FConcertSession_CustomResponse& Response)
		{
			// Handle the response
			CommonHandleCustomResponse(Response, Handler);
		});
}

void FConcertServerSession::HandleRemoteConnectionChanged(const FConcertEndpointContext& RemoteEndpointContext, EConcertRemoteEndpointConnection Connection)
{
	if (Connection == EConcertRemoteEndpointConnection::TimedOut || Connection == EConcertRemoteEndpointConnection::ClosedRemotely)
	{
		// Find the client in our list
		FSessionClient SessionClient;
		if (SessionClients.RemoveAndCopyValue(RemoteEndpointContext.EndpointId, SessionClient))
		{
			OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Disconnected, SessionClient.ClientInfo);

			UE_LOGF(LogConcert, Display, "User '%ls' (Endpoint: %ls) left Concert session '%ls' (Id: %ls, Owner: %ls) due to %ls.", 
				*SessionClient.ClientInfo.ClientInfo.UserName, 
				*SessionClient.ClientInfo.ClientEndpointId.ToString(), 
				*SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName,
				(Connection == EConcertRemoteEndpointConnection::TimedOut ? TEXT("time-out") : TEXT("the remote peer closing the connection"))
			);

			// Send client disconnection notification to other clients
			SendClientListUpdatedEvent();
		}
	}
}

void FConcertServerSession::HandleDiscoverAndJoinSessionEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_DiscoverAndJoinSessionEvent* Message = Context.GetMessage<FConcertSession_DiscoverAndJoinSessionEvent>();

	// If this this isn't a join request for this session, discard the message
	if (Message->SessionServerEndpointId != SessionInfo.ServerEndpointId)
	{
		return;
	}

	// This should not trigger, it would mean that the server discovery mechanism, allowed a mismatched protocol version
	check(Message->ConcertProtocolVersion == EConcertMessageVersion::LatestVersion);

	FConcertSession_JoinSessionResultEvent JoinReply;
	JoinReply.ConcertProtocolVersion = EConcertMessageVersion::LatestVersion;
	JoinReply.SessionServerEndpointId = SessionInfo.ServerEndpointId;

	const FConcertSessionInfo& InSessionInfo = GetSessionInfo();
	if (SessionClients.Contains(Context.SenderConcertEndpointId))
	{
		JoinReply.ConnectionResult = EConcertConnectionResult::AlreadyConnected;
		JoinReply.SessionClients = GetSessionClients();
	}
	else if (InSessionInfo.State == EConcertSessionState::Transient)
	{
		JoinReply.ConnectionResult = EConcertConnectionResult::ConnectionRefused;
	}
		// TODO: check connection requirement
	else // if (CheckConnectionRequirement(Message->ClientInfo))
	{
		// Accept the connection
		JoinReply.ConnectionResult = EConcertConnectionResult::ConnectionAccepted;
		JoinReply.SessionClients = GetSessionClients();
	}

	// Send the reply before we invoke the delegate and notify of the client list to ensure that the client knows it's connected before it starts receiving other messages
	ServerSessionEndpoint->SendEvent(JoinReply, Context.SenderConcertEndpointId, EConcertMessageFlags::ReliableOrdered);

	if (JoinReply.ConnectionResult == EConcertConnectionResult::ConnectionAccepted)
	{
		// Add the client to the list
		const FSessionClient& SessionClient = SessionClients.Add(Context.SenderConcertEndpointId, FSessionClient{ FConcertSessionClientInfo{ Context.SenderConcertEndpointId, Message->ClientInfo }, MakeShared<FConcertScratchpad, ESPMode::ThreadSafe>() });
		OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Connected, SessionClient.ClientInfo);

		UE_LOGF(LogConcert, Display, "User '%ls' (Endpoint: %ls) joined Concert session '%ls' (Id: %ls, Owner: %ls).", *SessionClient.ClientInfo.ClientInfo.UserName, *SessionClient.ClientInfo.ClientEndpointId.ToString(), *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);

		// Send client connection notification
		SendClientListUpdatedEvent();
	}
}

void FConcertServerSession::HandleLeaveSessionEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_LeaveSessionEvent* Message = Context.GetMessage<FConcertSession_LeaveSessionEvent>();

	// If this isn't a connection request for this session, discard the message
	if (Message->SessionServerEndpointId != SessionInfo.ServerEndpointId)
	{
		return;
	}

	// Find the client in our list
	FSessionClient SessionClient;
	if (SessionClients.RemoveAndCopyValue(Context.SenderConcertEndpointId, SessionClient))
	{
		OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Disconnected, SessionClient.ClientInfo);

		UE_LOGF(LogConcert, Display, "User '%ls' (Endpoint: %ls) left Concert session '%ls' (Id: %ls, Owner: %ls) by request.", *SessionClient.ClientInfo.ClientInfo.UserName, *SessionClient.ClientInfo.ClientEndpointId.ToString(), *SessionInfo.SessionName, *SessionInfo.SessionId.ToString(), *SessionInfo.OwnerUserName);

		// Send client disconnection notification to other clients
		SendClientListUpdatedEvent();
	}
}

void FConcertServerSession::HandleUpdateClientInfoEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_UpdateClientInfoEvent* Message = Context.GetMessage<FConcertSession_UpdateClientInfoEvent>();

	// Find the clients to update.
	check(Message->SessionClient.ClientEndpointId == Message->ConcertEndpointId);
	if (FSessionClient* SessionClient = SessionClients.Find(Message->ConcertEndpointId))
	{
		// Notify local observers about the change.
		SessionClient->ClientInfo.ClientInfo = Message->SessionClient.ClientInfo;
		OnSessionClientChangedDelegate.Broadcast(*this, EConcertClientStatus::Updated, SessionClient->ClientInfo);

		// Notify other clients about the change (except the one who initiated the event)
		for (const FConcertSessionClientInfo& Client : GetSessionClients())
		{
			if (Client.ClientEndpointId != Message->SessionClient.ClientEndpointId)
			{
				ServerSessionEndpoint->SendEvent(*Message, Client.ClientEndpointId, EConcertMessageFlags::ReliableOrdered);
			}
		}
	}
}

void FConcertServerSession::HandleCustomEvent(const FConcertMessageContext& Context)
{
	const FConcertSession_CustomEvent* Message = Context.GetMessage<FConcertSession_CustomEvent>();

	// Process or forward this event
	for (const FGuid& DestinationEndpointId : Message->DestinationEndpointIds)
	{
		if (DestinationEndpointId == SessionInfo.ServerEndpointId)
		{
			// Handle locally
			CommonHandleCustomEvent(Context);
		}
		else if (const FSessionClient* Client = SessionClients.Find(DestinationEndpointId))
		{
			// Forward onto the client
			ServerSessionEndpoint->SendEvent(*Message, Client->ClientInfo.ClientEndpointId, Message->IsReliable() ? EConcertMessageFlags::ReliableOrdered : EConcertMessageFlags::None, Context.Annotations);
		}
	}
}

TFuture<FConcertSession_CustomResponse> FConcertServerSession::HandleCustomRequest(const FConcertMessageContext& Context)
{
	const FConcertSession_CustomRequest* Message = Context.GetMessage<FConcertSession_CustomRequest>();

	if (Message->DestinationEndpointId == SessionInfo.ServerEndpointId)
	{
		// Handle locally
		return CommonHandleCustomRequest(Context);
	}
	else if (const FSessionClient* Client = SessionClients.Find(Message->DestinationEndpointId))
	{
		// Forward onto the client
		return ServerSessionEndpoint->SendRequest<FConcertSession_CustomRequest, FConcertSession_CustomResponse>(*Message, Client->ClientInfo.ClientEndpointId);
	}

	// Default response
	FConcertSession_CustomResponse ResponseData;
	ResponseData.ResponseCode = EConcertResponseCode::UnknownRequest;
	return FConcertSession_CustomResponse::AsFuture(MoveTemp(ResponseData));
}

void FConcertServerSession::SendClientListUpdatedEvent()
{
	// Notifying client connection is done by sending the current client list
	FConcertSession_ClientListUpdatedEvent ClientListUpdatedEvent;
	ClientListUpdatedEvent.SessionClients = GetSessionClients();
	for (const auto& SessionClientPair : SessionClients)
	{
		ServerSessionEndpoint->SendEvent(ClientListUpdatedEvent, SessionClientPair.Value.ClientInfo.ClientEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertServerSession::SendSessionNameChanged()
{
	// Notifying clients that the session name changed.
	FConcertSession_SessionRenamedEvent RenamedEvent;
	RenamedEvent.NewName = GetName();
	for (const auto& SessionClientPair : SessionClients)
	{
		ServerSessionEndpoint->SendEvent(RenamedEvent, SessionClientPair.Value.ClientInfo.ClientEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertServerSession::TickConnections(float DeltaSeconds)
{
	// External callback
	OnTickDelegate.Broadcast(*this, DeltaSeconds);
}
