// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertSession.h"
#include "ConcertMessages.h"
#include "Containers/Ticker.h"

class IConcertLocalEndpoint;
struct FConcertServerSettings;

/** Implementation of a Concert Server session */
class FConcertServerSession
	: public IConcertServerSession
	, public TSharedFromThis<FConcertServerSession>
	, private FConcertSessionCommonImpl
{
public:
	FConcertServerSession(const FConcertSessionInfo& InSessionInfo, const FConcertServerSettings& InSettings, TSharedPtr<IConcertLocalEndpoint> InServerSessionEndpoint, const FString& InSessionDirectory);
	virtual ~FConcertServerSession();

	virtual void Startup() override;
	virtual void Shutdown() override;

	virtual const FGuid& GetId() const override
	{
		return CommonGetId();
	}

	virtual const FString& GetName() const override
	{
		return CommonGetName();
	}

	virtual void SetName(const FString& NewName) override
	{
		CommonSetName(NewName);
		SendSessionNameChanged();
	}

	virtual FMessageAddress GetClientAddress(const FGuid& ClientEndpointId) const override
	{
		return ServerSessionEndpoint->GetRemoteAddress(ClientEndpointId);
	}

	void SetLastModifiedToNow()
	{
		SessionInfo.SetLastModifiedToNow();
	}

	virtual const FConcertSessionInfo& GetSessionInfo() const override
	{
		return CommonGetSessionInfo();
	}

	virtual TArray<FGuid> GetSessionClientEndpointIds() const override
	{
		return CommonGetSessionClientEndpointIds();
	}

	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override
	{
		return CommonGetSessionClients();
	}

	virtual bool FindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const override
	{
		return CommonFindSessionClient(EndpointId, OutSessionClientInfo);
	}

	virtual FConcertScratchpadRef GetScratchpad() const override
	{
		return CommonGetScratchpad();
	}

	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid& ClientEndpointId) const override
	{
		return CommonGetClientScratchpad(ClientEndpointId);
	}

	virtual FOnConcertServerSessionTick& OnTick() override;
	virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() override;
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() override;
	virtual FString GetSessionWorkingDirectory() const override;

protected:
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) override
	{
		return CommonRegisterCustomEventHandler(EventMessageType, Handler);
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override
	{
		CommonUnregisterCustomEventHandler(EventMessageType, EventHandle);
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override
	{
		CommonUnregisterCustomEventHandler(EventMessageType, EventHandler);
	}

	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override
	{
		CommonClearCustomEventHandler(EventMessageType);
	}

	virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequencedId = {}) override;
	
	virtual void InternalRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler) override
	{
		CommonRegisterCustomRequestHandler(RequestMessageType, Handler);
	}

	virtual void InternalUnregisterCustomRequestHandler(const FName& RequestMessageType) override
	{
		CommonUnregisterCustomRequestHandler(RequestMessageType);
	}

	virtual void InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler) override;

private:
	/** */
	void HandleRemoteConnectionChanged(const FConcertEndpointContext& RemoteEndpointContext, EConcertRemoteEndpointConnection Connection);

	/**  */
	void HandleDiscoverAndJoinSessionEvent(const FConcertMessageContext& Context);

	/**  */
	void HandleLeaveSessionEvent(const FConcertMessageContext& Context);

	/**  */
	void HandleUpdateClientInfoEvent(const FConcertMessageContext& Context);
	
	/** */
	void HandleCustomEvent(const FConcertMessageContext& Context);

	/** */
	TFuture<FConcertSession_CustomResponse> HandleCustomRequest(const FConcertMessageContext& Context);

	/**  */
	void SendClientListUpdatedEvent();

	/** */
	void SendSessionNameChanged();

	/**  */
	void TickConnections(float DeltaSeconds);

	/** This session endpoint where message are sent and received from. */
	IConcertLocalEndpointPtr ServerSessionEndpoint;

	/** Ticker for the session */
	FTSTicker::FDelegateHandle SessionTick;

	/** Callback for when a server session ticks */
	FOnConcertServerSessionTick OnTickDelegate;

	/** Callback for when a session client state changes */
	FOnConcertServerSessionClientChanged OnSessionClientChangedDelegate;

	/** The timespan at which session updates are processed. */
	const FTimespan SessionTickFrequency;

	/** The directory where this session will store its files */
	const FString SessionDirectory;
};
