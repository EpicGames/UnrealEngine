// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "ConcertServerSessionRepositories.generated.h"

/** Keep the information about a the session repository. */
USTRUCT()
struct FConcertServerSessionRepository
{
	GENERATED_BODY()

	FConcertServerSessionRepository() = default;

	FConcertServerSessionRepository(const FString& InRole, FGuid InRepositoryId, const FString& InWorkingDir, const FString& InSavedDir)
		: RepositoryId(InRepositoryId)
		, WorkingDir(InWorkingDir.Len() ? InWorkingDir : FPaths::ProjectIntermediateDir() / InRole)
		, SavedDir(InSavedDir.Len() ? InSavedDir : FPaths::ProjectSavedDir() / InRole)
	{
	}

	FConcertServerSessionRepository(const FString& RootPathname, FGuid InRepositoryId)
		: RepositoryId(InRepositoryId)
		, RepositoryRootDir(RootPathname)
		, WorkingDir(RootPathname / InRepositoryId.ToString() / TEXT("Live"))
		, SavedDir(RootPathname / InRepositoryId.ToString() / TEXT("Archive"))
	{
	}

	/** Return the working directory for a specific session */
	FString GetSessionWorkingDir(const FGuid& InSessionId) const
	{
		return WorkingDir / InSessionId.ToString();
	}

	/** Return the saved directory for a specific session */
	FString GetSessionSavedDir(const FGuid& InSessionId) const
	{
		return SavedDir / InSessionId.ToString();
	}

	/** The repository id.*/
	UPROPERTY()
	FGuid RepositoryId;

	/** The repository root directory. Can be empty is the working/archive dir are not stored in a standard repository. */
	UPROPERTY()
	FString RepositoryRootDir;

	/** This repository working directory. */
	UPROPERTY()
	FString WorkingDir;

	/** This repository saving directory. */
	UPROPERTY()
	FString SavedDir;

	/** The server process ID that mounted the workspace if bMounted is true. */
	UPROPERTY()
	int32 ProcessId = 0;

	/** Whether the workspace is mounted on a server.*/
	UPROPERTY()
	bool bMounted = false;
};

/** Tracks the session repositories across the server instances. */
USTRUCT()
struct FConcertServerSessionRepositoryDatabase
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FConcertServerSessionRepository> Repositories;
};
