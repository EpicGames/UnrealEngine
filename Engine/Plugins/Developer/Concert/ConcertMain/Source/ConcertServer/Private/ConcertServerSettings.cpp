// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertServerSettings)

UConcertServerConfig::UConcertServerConfig()
	: bCleanWorkingDir(false)
	, NumSessionsToKeep(-1)
{
	DefaultVersionInfo.Initialize(false /* bSupportMixedBuildTypes */);
}
