// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerUtil.h"
#include "ConcertMessageData.h"
#include "IConcertSession.h"
#include "IConcertServer.h"

namespace ConcertUtil::Private
{
	static TOptional<TPair<FConcertSessionClientInfo, TSharedPtr<IConcertServerSession>>> FindByClient(IConcertServer& Server, const FGuid& ClientEndpointId)
	{
		for (const TSharedPtr<IConcertServerSession>& ServerSession : Server.GetLiveSessions())
		{
			for (const FConcertSessionClientInfo& ClientInfo : ServerSession->GetSessionClients())
			{
				if (ClientEndpointId == ClientInfo.ClientEndpointId)
				{
					return {{ ClientInfo, ServerSession }};
				}
			}
		}

		return {};
	}
}
/*
TOptional<FConcertSessionClientInfo> ConcertUtil::GetConnectedClientInfo(IConcertServer& Server, const FGuid& ClientEndpointId)
{
	const TOptional<TPair<FConcertSessionClientInfo, TSharedPtr<IConcertServerSession>>> Result = Private::FindByClient(Server, ClientEndpointId);
	return Result
		? Result->Key
		: TOptional<FConcertSessionClientInfo>();
}
*/

TArray<FConcertSessionClientInfo> ConcertUtil::GetSessionClients(IConcertServer& Server, const FGuid& SessionId)
{
	TSharedPtr<IConcertServerSession> ServerSession = Server.GetLiveSession(SessionId);
	if (ServerSession)
	{
		return ServerSession->GetSessionClients();
	}
	return TArray<FConcertSessionClientInfo>();
}


TSharedPtr<IConcertServerSession> ConcertUtil::GetLiveSessionClientConnectedTo(IConcertServer& Server, const FGuid& ClientEndpointId)
{
	const TOptional<TPair<FConcertSessionClientInfo, TSharedPtr<IConcertServerSession>>> Result = Private::FindByClient(Server, ClientEndpointId);
	return Result
		? Result->Value
		: TSharedPtr<IConcertServerSession>();
}


