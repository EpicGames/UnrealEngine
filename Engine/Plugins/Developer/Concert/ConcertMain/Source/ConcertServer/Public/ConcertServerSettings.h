// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "ConcertSettings.h"
#include "ConcertTransportSettings.h"
#include "ConcertVersion.h"
#include "ConcertServerSettings.generated.h"

#define UE_API CONCERTSERVER_API

USTRUCT()
struct FConcertServerSettings
{
	GENERATED_BODY()

	/** The server will allow client to join potentially incompatible sessions */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Server Settings")
	bool bIgnoreSessionSettingsRestriction = false;

	/** The timespan at which session updates are processed. */
	UPROPERTY(config, EditAnywhere, DisplayName="Session Tick Frequency", AdvancedDisplay, Category="Server Settings", meta=(ForceUnits=s))
	int32 SessionTickFrequencySeconds = 1;
};

UCLASS(MinimalAPI, config=Engine)
class UConcertServerConfig : public UObject
{
	GENERATED_BODY()
public:
	UE_API UConcertServerConfig();

	/**
	 * If true, instruct the server to auto-archive sessions that were left in the working directory because the server did not exit properly rather than
	 * restoring them as 'live' (the default).
	 */
	UPROPERTY(config)
	bool bAutoArchiveOnReboot = false;

	/**
	 * If true, instruct the server to auto-archive live sessions on shutdown.
	 */
	UPROPERTY(config)
	bool bAutoArchiveOnShutdown = true;

	/**
	 * Clean server sessions working directory when booting
	 * Can be specified on the server cmd with `-CONCERTCLEAN`
	 */
	UPROPERTY(config, EditAnywhere, Category="Server Settings")
	bool bCleanWorkingDir;

	/**
	 * Number of archived sessions to keep when booting, or <0 to keep all archived sessions
	 */
	UPROPERTY(config, EditAnywhere, Category="Server Settings")
	int32 NumSessionsToKeep;

	/** 
	 * Name of the server, or empty to use the default name.
	 * Can be specified on the server cmd with `-CONCERTSERVER=`
	 */
	UPROPERTY(config, EditAnywhere, Category="Server Settings")
	FString ServerName;

	/** 
	 * Name of the default session created on the server.
	 * Can be specified on the server cmd with `-CONCERTSESSION=`
	 */
	UPROPERTY(config, EditAnywhere, Category="Session Settings")
	FString DefaultSessionName;

	/** 
	 * A set of keys identifying the clients that can discover and access the server. If empty, the server can be discovered and used by any clients.
	 */
	UPROPERTY(config)
	TSet<FString> AuthorizedClientKeys;

	/**
	 * Name of the default session to restore on the server.
	 * Set the name of the desired save to restore its content in your session.
	 * Leave this blank if you want to create an empty session.
	 * Can be specified on the editor cmd with `-CONCERTSESSIONTORESTORE=`.
	 */
	UPROPERTY(config, EditAnywhere, Category="Session Settings")
	FString DefaultSessionToRestore;

	/** 
	 * The version string for the default server created.
	 * Can be specified on the server cmd with `-CONCERTVERSION=`
	 */
	UPROPERTY()
	FConcertSessionVersionInfo DefaultVersionInfo;

	/** Default server session settings */
	UPROPERTY(config, EditAnywhere, Category="Session Settings")
	FConcertSessionSettings DefaultSessionSettings;

	/** Server & server session settings */
	UPROPERTY(config, EditAnywhere, Category="Server Settings", meta=(ShowOnlyInnerProperties))
	FConcertServerSettings ServerSettings;

	/** Endpoint settings passed down to endpoints on creation */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Endpoint Settings", meta=(ShowOnlyInnerProperties))
	FConcertEndpointSettings EndpointSettings;

	/** The default directory where the server keeps the live session files. Can be specified on the server command line with `-CONCERTWORKINGDIR=`*/
	UPROPERTY(config)
	FString WorkingDir;

	/** The default directory where the server keeps the archived session files. Can be specified on the server command line with `-CONCERTSAVEDDIR=`*/
	UPROPERTY(config)
	FString ArchiveDir;

	/** The root directory where the server creates new session repositories (unless the client request specifies its own root). If empty or invalid, the server will use a default. */
	UPROPERTY(config)
	FString SessionRepositoryRootDir;

	/** If neither of WorkingDir and ArchiveDir are set, determine whether the server should mount a standard default session repository where new session will be created. */
	UPROPERTY(config)
	bool bMountDefaultSessionRepository = true;

	/**
	 * Specifies if we should use a file sharing service for larger assets that are saved in a Multi-user session.
	 * Note the specified folder must be visible to all clients.
	 *
	 * This can be set on the command line using -CONCERTFILESHARINGENABLED
	 */
	UPROPERTY(config, EditAnywhere, Category = "File Sharing")
	bool bEnableFileSharing = false;

	/**
	 * The path for that shares files across clients. This path must be visible to all nodes. 
	 *
	 * This can be set on the command line using -CONCERTFILESHAREPATH=<FullPathName>
	 */
	UPROPERTY(config, EditAnywhere, meta=(EditCondition="bEnableFileSharing"), Category = "File Sharing")
	FDirectoryPath RootFileSharingPath;
};

#undef UE_API
