// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"
#include "ConcertMessageData.h"

#include "IConcertSession.h"

class IConcertServer;

namespace ConcertUtil
{
	/**
	 * Get the list of clients for a session
	 * @param Server The server to look on
	 * @param SessionId The session ID
	 * @return A list of clients connected to the session
	 */
	CONCERTSERVER_API TArray<FConcertSessionClientInfo> GetSessionClients(IConcertServer& Server, const FGuid& SessionId);

	/**
	 * Retrieves the server session object from the given connected client.
	 * @param Server The server to look on
	 * @param ClientEndpointId The client's endpoint ID
	 * @return The session the client is connected to if any
	 */
	CONCERTSERVER_API TSharedPtr<IConcertServerSession> GetLiveSessionClientConnectedTo(IConcertServer& Server, const FGuid& ClientEndpointId);

}
