// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertEndpoint.h"
#include "IConcertSession.h"

class UConcertServerConfig;
class IConcertFileSharingService;
struct FConcertClientInfo;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertServerSessionStartup, TWeakPtr<IConcertServerSession>);
DECLARE_MULTICAST_DELEGATE(FOnConcertServerStartup);
DECLARE_DELEGATE_RetVal_FourParams(bool, FOnConcertParticipantCanJoinSession, const FGuid& /*SessionId*/, const FGuid& /*EndpointId*/, const FConcertClientInfo& /*ClientInfo*/, FText* /*OutFailureReason*/);

/** Interface for Concert server */
class IConcertServer
{
public:
	virtual ~IConcertServer() = default;

	/**
	 * Get the role of this server (eg, MultiUser, DisasterRecovery, etc)
	 */
	virtual const FString& GetRole() const = 0;

	/**
	 * Configure the Concert settings and its information
	 */
	virtual void Configure(const UConcertServerConfig* ServerConfig) = 0;
	
	/** 
	 * Return true if the server has been configured.
	 */
	virtual bool IsConfigured() const = 0;

	/**
	 * Return The configuration of this server, or null if it hasn't been configured.
	 */
	virtual const UConcertServerConfig* GetConfiguration() const = 0;

	/**
	 * Get the server information set by Configure
	 */
	virtual const FConcertServerInfo& GetServerInfo() const = 0;

	/**
	 * Sets the handle to the file sharing service.  The server needs to verify that it can access the challenge provided by
	 * the client to confirm they both have access to the file sharing service. 
	 */
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) = 0;

	/**
	 * Get the current file sharing service held by the server. 
	 */
	virtual TSharedPtr<IConcertFileSharingService> GetFileSharingService() const = 0;		

	/**
	 * Gets all remote admin endpoint IDs.
	 * A remote admin endpoint is not connected to any session and communicating with the server otherwise, e.g. discovering sessions.
	 */
	virtual TArray<FConcertEndpointContext> GetRemoteAdminEndpoints() const = 0;
	
	/** Callback when a remote admin endpoint connection changes. */
	virtual FOnConcertRemoteEndpointConnectionChanged& OnRemoteEndpointConnectionChanged() = 0;

	/** Callback to check whether a participant should be allowed to join a session. */
	virtual FOnConcertParticipantCanJoinSession& OnConcertParticipantCanJoinSession() = 0;
	
	/**
	 * Gets the address of a remote admin endpoint, i.e. a client that is sending FConcertEndpointDiscoveryEvents. 
	 */
	virtual FMessageAddress GetRemoteAddress(const FGuid& AdminEndpointId) const = 0;

	/** Callback when a message has been acknowledged by a remote endpoint */
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() = 0;

	/**
	 *	Returns if the server has already been started up.
	 */
	virtual bool IsStarted() const = 0;

	/**
	 * Startup the server, this can be called multiple time
	 * Configure needs to be called before startup
	 * @return true if the server was properly started or already was
	 */
	virtual void Startup() = 0;

	/**
	 * Shutdown the server, this can be called multiple time with no ill effect.
 	 * However it depends on the UObject system so need to be called before its exit.
	 */
	virtual void Shutdown() = 0;

	/**
	 * Get the ID of a live session from its name.
	 * @return The ID of the session, or an invalid GUID if it couldn't be found.
	 */
	virtual FGuid GetLiveSessionIdByName(const FString& InName) const = 0;

	/**
	 * Get the ID of an archived session from its name.
	 * @return The ID of the session, or an invalid GUID if it couldn't be found.
	 */
	virtual FGuid GetArchivedSessionIdByName(const FString& InName) const = 0;

	/**
	 * Create a session description for this server
	 */
	virtual FConcertSessionInfo CreateSessionInfo() const = 0;

	/**
	 * Get the live session information list
	 */
	virtual	TArray<FConcertSessionInfo> GetLiveSessionInfos() const = 0;

	/**
	 * Gets the archived session information list
	 */
	virtual TArray<FConcertSessionInfo> GetArchivedSessionInfos() const = 0;

	/**
	 * Get all live server sessions
	 * @return array of server sessions
	 */
	virtual TArray<TSharedPtr<IConcertServerSession>> GetLiveSessions() const = 0;

	/**
	 * Get a live server session
	 * @param SessionId The ID of the session we want
	 * @return the server session or an invalid pointer if no session was found
	 */
	virtual TSharedPtr<IConcertServerSession> GetLiveSession(const FGuid& SessionId) const = 0;

	/**
	 * Gets an archived session info
	 * @param SessionId The ID of the session we want
	 * @return The info of the archived session if found
	 */
	virtual TOptional<FConcertSessionInfo> GetArchivedSessionInfo(const FGuid& SessionId) const = 0;

	/** 
	 * Create a new Concert server session based on the passed session info
	 * @param SessionInfo The information about the session to create.
	 * @param OutFailureReason The reason the operation fails if the function returns false, undefined otherwise.
	 * @return the created server session
	 */
	virtual TSharedPtr<IConcertServerSession> CreateSession(const FConcertSessionInfo& SessionInfo, FText& OutFailureReason) = 0;

	/** 
	 * Create a new live server session from another session. The source session can be an archive or a live session.
	 * @param SrcSessionId The ID of the session to copy. If the session is archived, its equivalent to restore it.
	 * @param NewSessionInfo The information about the new session to create.
	 * @param SessionFilter The filter controlling which activities should be copied over the new session.
	 * @param OutFailureReason The reason the operation fails if the function returns null, undefined otherwise.
	 * @return the created server session
	 * @note This is equivalent to archiving and restoring a session, but faster as it skips one copy of the session (possibly serveral GB).
	 */
	virtual TSharedPtr<IConcertServerSession> CopySession(const FGuid& SrcSessionId, const FConcertSessionInfo& NewSessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason) = 0;

	/**
	 * Restore an archived Concert server session based on the passed session info
	 * @param SessionId The ID of the session to restore
	 * @param SessionInfo The information about the session to create from the archive.
	 * @param SessionFilter The filter controlling which activities from the session should be restored.
	 * @param OutFailureReason The reason the operation fails if the function returns false, undefined otherwise.
	 * @return the restored server session
	 */
	virtual TSharedPtr<IConcertServerSession> RestoreSession(const FGuid& SessionId, const FConcertSessionInfo& SessionInfo, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason) = 0;

	/**
	 * Archive a Concert session on the server.
	 * @param SessionId The ID of the session to archive
	 * @param ArchiveNameOverride The name override to give to the archived session.
	 * @param SessionFilter The filter controlling which activities from the session should be archived.
	 * @param OutFailureReason The reason the operation fails if the function returns false, undefined otherwise.
	 * @param ArchiveSessionIdOverride The ID the archived session is supposed to have
	 * @return The ID of the archived session on success, or an invalid GUID otherwise.
	 */
	virtual FGuid ArchiveSession(const FGuid& SessionId, const FString& ArchiveNameOverride, const FConcertSessionFilter& SessionFilter, FText& OutFailureReason, FGuid ArchiveSessionIdOverride = FGuid::NewGuid()) = 0;

	/**
	 * Copy the session data to a destination folder for external usage.
	 * @param SessionId The ID of the session to export. Can be a live or archived session.
	 * @param SessionFilter The filter controlling which activities from the session should be archived.
	 * @param DestDir The directory where the exported files should be copied. (Must exist)
	 * @param bAnonymizeData True to obfuscate the object and package names stored in the database. Ignored if FConcertSessionFilter::bMetaDataOnly is false.
	 * @param OutFailureReason The reason why exporting the session would fail.
	 * @return True if the session files were exported successfully, false otherwise.
	 */
	virtual bool ExportSession(const FGuid& SessionId, const FConcertSessionFilter& SessionFilter, const FString& DestDir, bool bAnonymizeData, FText& OutFailureReason) = 0;

	/**
	 * Rename a live or archived Concert session on the server. The server automatically detects if the specified session Id is a live or an
	 * archived session.
	 * @param SessionId The ID of the session to rename
	 * @param NewName The new session name.
	 * @param OutFailureReason The reason the operation fails if the function returns false, undefined otherwise.
	 * @return True if the session was renamed.
	 */
	virtual bool RenameSession(const FGuid& SessionId, const FString& NewName, FText& OutFailureReason) = 0;

	/**
	 * Destroy a live or archived Concert server session. The server automatically detects if the specified session Id is a live or an
	 * archived session.
	 * @param SessionId The name of the session to destroy
	 * @param OutFailureReason The reason the operation fails if the function returns false, undefined otherwise.
	 * @return true if the session was found and destroyed
	 */
	virtual bool DestroySession(const FGuid& SessionId, FText& OutFailureReason) = 0;

	/**
	 * Callback for when a concert server session has been created.
	 */
	virtual FOnConcertServerSessionStartup& OnConcertServerSessionStartup() = 0;

	/**
	 * Callback for when a concert server has been created.
	 */
	virtual FOnConcertServerStartup& OnConcertServerStartup() = 0;
};
