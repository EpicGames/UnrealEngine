// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IConcertServer;
class IConcertServerEventSink;

struct FConcertSessionFilter;

typedef TSharedPtr<IConcertServer, ESPMode::ThreadSafe> IConcertServerPtr;
typedef TSharedRef<IConcertServer, ESPMode::ThreadSafe> IConcertServerRef;

/**
 * Interface for the Main Concert Server module.
 */
class IConcertServerModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IConcertServerModule& Get()
	{
		static const FName ModuleName = "ConcertServer";
		return FModuleManager::LoadModuleChecked<IConcertServerModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "ConcertServer";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Create a server that will perform a certain role (eg, MultiUser, DisasterRecovery, etc)
	 * @param InRole The role to create
	 * @param InAutoArchiveSessionFilter The session filter to apply when auto-archiving sessions
	 * @param InEventSink Sink functions for events that the server can emit
	 * @return The server
	 */
	virtual IConcertServerRef CreateServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter, IConcertServerEventSink* InEventSink) = 0;

};
