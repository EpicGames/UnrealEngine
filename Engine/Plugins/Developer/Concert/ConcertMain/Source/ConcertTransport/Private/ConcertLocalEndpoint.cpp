// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLocalEndpoint.h"
#include "Async/TaskGraphInterfaces.h"
#include "ConcertLogGlobal.h"

#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Algo/Transform.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Containers/Ticker.h"
#include "Misc/DateTime.h"
#include "Stats/Stats.h"

LLM_DEFINE_TAG(Concert_ConcertLocalEndpoint);

class FConcertLocalEndpointKeepAliveRunnable : public FRunnable
{
public:
	explicit FConcertLocalEndpointKeepAliveRunnable(FConcertLocalEndpoint* InLocalEndpoint, const TCHAR* InMessageEndpointName)
		: bIsRunning(false)
		, bStopRequested(false)
		, LocalEndpoint(InLocalEndpoint)
		, Thread(FRunnableThread::Create(this, InMessageEndpointName))
	{
		check(LocalEndpoint);
	}

	~FConcertLocalEndpointKeepAliveRunnable()
	{
		if (Thread)
		{
			Thread->Kill(true);
			delete Thread;
			Thread = nullptr;
		}
	}

	bool IsRunning() const
	{
		return Thread && bIsRunning;
	}

	//~ FRunnable Interface
	virtual bool Init() override
	{
		bIsRunning = true;
		return true;
	}

	virtual uint32 Run() override
	{
		while (!bStopRequested)
		{
			const FDateTime UtcNow = FDateTime::UtcNow();
			LocalEndpoint->SendKeepAlives(UtcNow);
			FPlatformProcess::SleepNoStats(1.0f);
		}
		return 0;
	}

	virtual void Stop() override
	{
		bStopRequested = true;
	}

	virtual void Exit() override
	{
		bIsRunning = false;
	}

private:
	TAtomic<bool> bIsRunning;
	TAtomic<bool> bStopRequested;
	FConcertLocalEndpoint* LocalEndpoint;
	FRunnableThread* Thread;
};

FConcertLocalEndpoint::FConcertLocalEndpoint(const FString& InEndpointFriendlyName, const FConcertEndpointSettings& InEndpointSettings, const FConcertTransportLoggerFactory& InLogFactory)
	: EndpointContext(FConcertEndpointContext{ FGuid::NewGuid(), InEndpointFriendlyName })
	, NextReliableChannelId(FConcertMessageData::UnreliableChannelId + 1)
	, bIsHandlingMessage(false)
	, Settings(InEndpointSettings)
	, Logger(InLogFactory ? InLogFactory(EndpointContext) : IConcertTransportLoggerPtr())
{
	if (Settings.bEnableLogging)
	{
		Logger.StartLogging();
	}

	const FString MessageEndpointName = FString::Printf(TEXT("Concert%sEndpoint"), *InEndpointFriendlyName);
	MessageEndpoint = FMessageEndpoint::Builder(*MessageEndpointName)
		.ReceivingOnAnyThread()
		.WithCatchall(this, &FConcertLocalEndpoint::InternalHandleMessage)
		.NotificationHandling(FOnBusNotification::CreateRaw(this, &FConcertLocalEndpoint::InternalHandleBusNotification));
	check(MessageEndpoint.IsValid());

	KeepAliveRunnable = MakeUnique<FConcertLocalEndpointKeepAliveRunnable>(this, *MessageEndpointName);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FConcertLocalEndpoint::HandleTick), 0.0f);
}

FConcertLocalEndpoint::~FConcertLocalEndpoint()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

	KeepAliveRunnable.Reset();

	const FDateTime UtcNow = FDateTime::UtcNow();
	for (auto It = RemoteEndpoints.CreateIterator(); It; ++It)
	{
		const FGuid EndpointId = It->Key;
		FConcertRemoteEndpointPtr RemoteEndpoint = It->Value;

		check(RemoteEndpoint.IsValid());

		// TODO: Disconnected handler:
		Logger.LogRemoteEndpointTimeOut(EndpointId, UtcNow);
		SendEndpointClosed(RemoteEndpoint.ToSharedRef(), UtcNow);
		It.RemoveCurrent();
	}

	// Disable the Endpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint)
	{
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}

	Logger.StopLogging();
}

const FConcertEndpointContext& FConcertLocalEndpoint::GetEndpointContext() const
{
	return EndpointContext;
}

TArray<FConcertEndpointContext> FConcertLocalEndpoint::GetRemoteEndpoints() const
{
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
	TArray<FConcertEndpointContext> Result;
	Algo::Transform(RemoteEndpoints, Result, [](const TPair<FGuid, FConcertRemoteEndpointPtr>& RemoteEndpoint)
	{
		return RemoteEndpoint.Value->GetEndpointContext();
	});
	return Result;
}

FMessageAddress FConcertLocalEndpoint::GetRemoteAddress(const FGuid& ConcertEndpointId) const
{
	const FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(ConcertEndpointId);
	return RemoteEndpoint
		? RemoteEndpoint->GetAddress()
		: FMessageAddress();
}

FOnConcertRemoteEndpointConnectionChanged& FConcertLocalEndpoint::OnRemoteEndpointConnectionChanged()
{
	return OnRemoteEndpointConnectionChangedDelegate;
}

FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& FConcertLocalEndpoint::OnConcertMessageAcknowledgementReceived()
{
	return OnConcertMessageAcknowledgementReceivedDelegate;
}

void FConcertLocalEndpoint::InternalAddRequestHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IConcertRequestHandler>& Handler)
{
	RequestHandlers.Add(RequestMessageType, Handler);
}

void FConcertLocalEndpoint::InternalAddEventHandler(const FTopLevelAssetPath& EventMessageType, const TSharedRef<IConcertEventHandler>& Handler)
{
	EventHandlers.Add(EventMessageType, Handler);
}

void FConcertLocalEndpoint::InternalRemoveRequestHandler(const FTopLevelAssetPath& RequestMessageType)
{
	RequestHandlers.Remove(RequestMessageType);
}

void FConcertLocalEndpoint::InternalRemoveEventHandler(const FTopLevelAssetPath& EventMessageType)
{
	EventHandlers.Remove(EventMessageType);
}

void FConcertLocalEndpoint::InternalSubscribeToEvent(const FTopLevelAssetPath& EventMessageType)
{
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe(EventMessageType, FMessageScopeRange::AtLeast(EMessageScope::Thread));
	}
}

void FConcertLocalEndpoint::InternalUnsubscribeFromEvent(const FTopLevelAssetPath& EventMessageType)
{
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Unsubscribe(EventMessageType);
	}
}

void FConcertLocalEndpoint::InternalQueueRequest(const TSharedRef<IConcertRequest>& Request, const FGuid& Endpoint)
{
	// Fill sending info
	SetMessageSendingInfo(Request);

	// Get the remote endpoint we want to send to.
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Endpoint);
	if (!RemoteEndpoint.IsValid())
	{
		Logger.LogTimeOut(Request, Endpoint, Request->GetCreationDate());
		Request->TimeOut();
		return;
	}

	// Queue the request since its considered reliable
	RemoteEndpoint->QueueMessageToSend(Request);
	if (RemoteEndpoint->HasReliableChannel())
	{
		SendMessage(Request, RemoteEndpoint.ToSharedRef(), Request->GetCreationDate());
	}
}

void FConcertLocalEndpoint::InternalQueueResponse(const TSharedRef<IConcertResponse>& Response, const FGuid& Endpoint)
{
	// Get the remote endpoint we want to send to.
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Endpoint);
	if (!RemoteEndpoint.IsValid())
	{
		// If we are about to send back to a unknown endpoint, the message should have been ignored earlier
		check(false);
		return;
	}

	// Queue the response since its considered reliable
	RemoteEndpoint->QueueMessageToSend(Response);
	if (RemoteEndpoint->HasReliableChannel())
	{
		SendMessage(Response, RemoteEndpoint.ToSharedRef(), Response->GetCreationDate());
	}
}

void FConcertLocalEndpoint::InternalQueueEvent(const TSharedRef<IConcertEvent>& Event, const FGuid& Endpoint, EConcertMessageFlags Flags)
{
	// Fill sending info
	SetMessageSendingInfo(Event);

	// Otherwise Get the remote endpoint we want to send to.
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Endpoint);
	if (!RemoteEndpoint.IsValid())
	{
		Logger.LogTimeOut(Event, Endpoint, Event->GetCreationDate());
		Event->TimeOut();
		return;
	}

	// If the event is reliable queue in that remote endpoint list
	if (EnumHasAnyFlags(Flags, EConcertMessageFlags::ReliableOrdered))
	{
		RemoteEndpoint->QueueMessageToSend(Event);
		if (!RemoteEndpoint->HasReliableChannel())
		{
			return;
		}
	}

	SendMessage(Event, RemoteEndpoint.ToSharedRef(), Event->GetCreationDate(), Flags);
}

void FConcertLocalEndpoint::InternalPublishEvent(const TSharedRef<IConcertEvent>& Event)
{
	// Fill sending info
	SetMessageSendingInfo(Event);

	// Publish the event
	PublishMessage(Event);
}

FConcertRemoteEndpointRef FConcertLocalEndpoint::CreateRemoteEndpoint(const FConcertEndpointContext& InEndpointContext, const FDateTime& InLastReceivedMessageTime, const FMessageAddress& InRemoteAddress)
{
	const uint16 NewRemoteEndpointChannelId = NextReliableChannelId++;
	if (NextReliableChannelId == FConcertMessageData::UnreliableChannelId)
	{
		++NextReliableChannelId;
	}

	const FTimespan RemoteTimeoutspan(0, 0, Settings.RemoteEndpointTimeoutSeconds);

	//We are optimistic and we use our own TimeoutSpan for the remote EndPoint. Until it's set by the handshake.
	FConcertRemoteEndpointRef NewRemoteEndpoint = MakeShared<FConcertRemoteEndpoint, ESPMode::ThreadSafe>(InEndpointContext, NewRemoteEndpointChannelId, InLastReceivedMessageTime, RemoteTimeoutspan, InRemoteAddress, Logger.GetLogger());
	{
		FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
		RemoteEndpoints.Add(InEndpointContext.EndpointId, NewRemoteEndpoint);
	}

	{
		FScopeLock RemoteAddressesLock(&RemoteAddressesCS);
		RemoteAddresses.Add(InRemoteAddress);
	}
	
	NewRemoteEndpoint->OnConcertMessageAcknowledgementReceived().AddLambda(
		[this](const FConcertEndpointContext& Context, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& AckMessageContext)
		{
			OnConcertMessageAcknowledgementReceivedDelegate.Broadcast(GetEndpointContext(), Context, AckedMessage, AckMessageContext);
		});
	return NewRemoteEndpoint;
}

FConcertRemoteEndpointPtr FConcertLocalEndpoint::FindRemoteEndpoint(const FGuid& InEndpointId) const
{
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
	return RemoteEndpoints.FindRef(InEndpointId);
}

FConcertRemoteEndpointPtr FConcertLocalEndpoint::FindRemoteEndpoint(const FMessageAddress& InAddress) const
{
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
	for (const auto& RemoteEndpointPair : RemoteEndpoints)
	{
		if (RemoteEndpointPair.Value->GetAddress() == InAddress)
		{
			return RemoteEndpointPair.Value;
		}
	}
	return nullptr;
}

void FConcertLocalEndpoint::RemoveRemoteEndpoint(const FMessageAddress& EndpointAddress)
{
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);

	TArray<FConcertRemoteEndpointPtr> Endpoints;
	RemoteEndpoints.GenerateValueArray(Endpoints);

	for (FConcertRemoteEndpointPtr RemoteEndpoint : Endpoints)
	{
		if (RemoteEndpoint->GetAddress() == EndpointAddress)
		{
			const FGuid& EndpointId = RemoteEndpoint->GetEndpointContext().EndpointId;

			PendingRemoteEndpointConnectionChangedEvents.Add(MakeTuple(RemoteEndpoint->GetEndpointContext(), EConcertRemoteEndpointConnection::ClosedRemotely));
			Logger.LogRemoteEndpointClosure(EndpointId, FDateTime::UtcNow());
			RemoteEndpoints.Remove(EndpointId);

			FScopeLock RemoteAddressesLock(&RemoteAddressesCS);
			RemoteAddresses.Remove(EndpointAddress);

			break;
		}
	}
}

bool FConcertLocalEndpoint::HandleTick(float DeltaTime)
{
	LLM_SCOPE_BYTAG(Concert_ConcertLocalEndpoint);
	SCOPED_CONCERT_TRACE(FConcertLocalEndpoint_HandleTick);


	// Flush the task graph to grab any pending messages
	// We put a dummy fence task into the queue to avoid potentially waiting indefinitely if other threads keep adding game thread events
	if (!FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread))
	{
		DECLARE_CYCLE_STAT(TEXT("FConcertLocalEndpoint.HandleTick"), STAT_FConcertLocalEndpoint_HandleTick, STATGROUP_TaskGraphTasks);
		FGraphEventRef FenceHandle = FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate(), GET_STATID(STAT_FConcertLocalEndpoint_HandleTick), nullptr, ENamedThreads::GameThread);
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(FenceHandle, ENamedThreads::GameThread);
	}

	const FDateTime UtcNow = FDateTime::UtcNow();
	HandleInboundMessages(UtcNow);

	ProcessQueuedReceivedMessages(UtcNow);
	TimeoutRemoteEndpoints(UtcNow);

	TArray<FConcertRemoteEndpointPtr> RemoteEndpointArray;
	{
		FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
		RemoteEndpoints.GenerateValueArray(RemoteEndpointArray);
	}

	PurgeOldReceivedMessages(RemoteEndpointArray, UtcNow);
	SendAcks(RemoteEndpointArray, UtcNow);
	ResendPendingMessages(RemoteEndpointArray, UtcNow);

	if (!KeepAliveRunnable->IsRunning())
	{
		SendKeepAlives(UtcNow);
	}

	for (const auto& PendingRemoteEndpointConnectionChangedEvent : PendingRemoteEndpointConnectionChangedEvents)
	{
		OnRemoteEndpointConnectionChangedDelegate.Broadcast(PendingRemoteEndpointConnectionChangedEvent.Get<0>(), PendingRemoteEndpointConnectionChangedEvent.Get<1>());
	}
	PendingRemoteEndpointConnectionChangedEvents.Reset();

	Logger.FlushLog();

	return true;
}

void FConcertLocalEndpoint::QueueAck(const FConcertMessageContext& ConcertContext)
{
	// If the message is reliable, queue an acknowledgment
	if (ConcertContext.Message->IsReliable())
	{
		FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(ConcertContext.SenderConcertEndpointId);
		if (RemoteEndpoint.IsValid())
		{
			RemoteEndpoint->QueueAcknowledgmentToSend(ConcertContext.Message->MessageId);
		}
	}
}

void FConcertLocalEndpoint::SendAcks(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow)
{
	for (const auto& RemoteEndpoint : InRemoteEndpoints)
	{
		check(RemoteEndpoint.IsValid());
		TOptional<FGuid> NextAcknowledgmentToSend = RemoteEndpoint->GetNextAcknowledgmentToSend();
		if (NextAcknowledgmentToSend.IsSet())
		{
			SendAck(NextAcknowledgmentToSend.GetValue(), RemoteEndpoint.ToSharedRef(), UtcNow);
		}
	}
}

void FConcertLocalEndpoint::SendAck(const FGuid& AcknowledgmentToSend, const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	FConcertAckData* Ack = FMessageEndpoint::MakeMessage<FConcertAckData>();
	Ack->ConcertEndpointId = EndpointContext.EndpointId;
	Ack->MessageId = FGuid::NewGuid();
	Ack->AckSendTimeTicks = UtcNow.GetTicks();
	Ack->SourceMessageId = AcknowledgmentToSend;

	// Update the last sent message time to this endpoint
	RemoteEndpoint->SetLastSentMessageTime(UtcNow);

	Logger.LogSendAck(*Ack, RemoteEndpoint->GetEndpointContext().EndpointId);

	MessageEndpoint->Send(
		Ack, // Should be deleted by MessageBus
		EMessageFlags::Reliable,
		nullptr, // No Attachment
		TArrayBuilder<FMessageAddress>().Add(RemoteEndpoint->GetAddress()),
		FTimespan::Zero(), // No Delay
		FDateTime::MaxValue() // No Expiration
	);
}

void FConcertLocalEndpoint::SendEndpointClosed(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	FConcertEndpointClosedData* EndpointClosed = FMessageEndpoint::MakeMessage<FConcertEndpointClosedData>();
	EndpointClosed->ConcertEndpointId = EndpointContext.EndpointId;
	EndpointClosed->MessageId = FGuid::NewGuid();

	// Update the last sent message time to this endpoint
	RemoteEndpoint->SetLastSentMessageTime(UtcNow);

	Logger.LogSendEndpointClosed(*EndpointClosed, RemoteEndpoint->GetEndpointContext().EndpointId, UtcNow);

	MessageEndpoint->Send(
		EndpointClosed, // Should be deleted by MessageBus
		EMessageFlags::None,
		nullptr, // No Attachment
		TArrayBuilder<FMessageAddress>().Add(RemoteEndpoint->GetAddress()),
		FTimespan::Zero(), // No Delay
		FDateTime::MaxValue() // No Expiration
	);
}

void FConcertLocalEndpoint::PublishMessage(const TSharedRef<IConcertMessage>& Message)
{
	if (!MessageEndpoint.IsValid())
	{
		// TODO: Error
		return;
	}

	Logger.LogPublish(Message);
	MessageEndpoint->Publish(
		Message->ConstructMessage(),
		Message->GetMessageType(),
		EMessageScope::Network,
		FTimespan::Zero(), // No Delay
		FDateTime::MaxValue() // No Expiration
	);
}

void FConcertLocalEndpoint::SendMessage(const TSharedRef<IConcertMessage>& Message, const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow, EConcertMessageFlags Flags)
{
	if (!MessageEndpoint.IsValid())
	{
		// TODO: Timeout the request & dequeue from remote endpoint
		return;
	}

	SCOPED_CONCERT_TRACE(FConcertLocalEndpoint_SendMessage);
	// Update the last sent message time to this endpoint
	RemoteEndpoint->SetLastSentMessageTime(UtcNow);

	Logger.LogSend(Message, RemoteEndpoint->GetEndpointContext().EndpointId);
	MessageEndpoint->Send(
		Message->ConstructMessage(), // Should be deleted by MessageBus
		Message->GetMessageType(),
		Message->IsReliable() ? EMessageFlags::Reliable : EMessageFlags::None,
		Message->GetAnnotations(),
		nullptr, // No Attachment
		TArrayBuilder<FMessageAddress>().Add(RemoteEndpoint->GetAddress()),
		FTimespan::Zero(), // No Delay
		FDateTime::MaxValue() // No Expiration
	);
}

void FConcertLocalEndpoint::HandleInboundMessages(const FDateTime& UtcNow)
{
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> InContext;
	while(InboundMessages.Dequeue(InContext))
	{
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context = InContext.ToSharedRef();
		const UScriptStruct* MessageTypeInfo = Context->GetMessageTypeInfo().Get();

		// Setup Context
		const FConcertMessageData* Message = (const FConcertMessageData*)Context->GetMessage();
		const FConcertMessageContext ConcertContext(Message->ConcertEndpointId, UtcNow, Message, MessageTypeInfo, Context->GetAnnotations());
		Logger.LogMessageReceived(ConcertContext, EndpointContext.EndpointId);

		// Special endpoint discovery message handling, process discovery before passing down the message
		if (MessageTypeInfo->IsChildOf(FConcertEndpointDiscoveryEvent::StaticStruct()))
		{
			ProcessEndpointDiscovery(ConcertContext, Context->GetSender());
		}

		if (MessageTypeInfo->IsChildOf(FConcertSendResendPending::StaticStruct()))
		{
			ForcePendingResend();
			continue;
		}

		// Special reliable handshake message handling, process then discard
		if (MessageTypeInfo->IsChildOf(FConcertReliableHandshakeData::StaticStruct()))
		{
			ProcessReliableHandshake(ConcertContext);
			continue;
		}

		QueueReceivedMessage(ConcertContext);
	}
}

void FConcertLocalEndpoint::ForcePendingResend()
{
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);
	for (auto It = RemoteEndpoints.CreateIterator(); It; ++It)
	{
		FConcertRemoteEndpointPtr RemoteEndpoint = It->Value;
		check(RemoteEndpoint.IsValid());
		RemoteEndpoint->MarkForResend();
	}
}

void FConcertLocalEndpoint::ProcessKeepAliveMessage(const TSharedPtr<IMessageContext, ESPMode::ThreadSafe>& Context, const FDateTime& UtcNow)
{
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Context->GetSender());
	if (RemoteEndpoint)
	{
		RemoteEndpoint->UpdateLastMessageReceivedTime(UtcNow);
	}
}

void FConcertLocalEndpoint::InternalHandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	LLM_SCOPE_BYTAG(Concert_ConcertLocalEndpoint);
	SCOPED_CONCERT_TRACE(FConcertLocalEndpoint_InternalHandleMessage);
	const UScriptStruct* MessageTypeInfo = Context->GetMessageTypeInfo().Get();

	if (!MessageTypeInfo)
	{
		// TODO: error?
		return;
	}

	if (!MessageTypeInfo->IsChildOf(FConcertMessageData::StaticStruct()))
	{
		// Not a Concert Message
		return;
	}

	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> ContextPtr(Context);
	if (MessageTypeInfo->IsChildOf(FConcertKeepAlive::StaticStruct()))
	{
		const FDateTime UtcNow = FDateTime::UtcNow();
		ProcessKeepAliveMessage(Context, UtcNow);
	}

	InboundMessages.Enqueue(MoveTemp(ContextPtr));
}

void FConcertLocalEndpoint::InternalHandleBusNotification(const FMessageBusNotification& Notification)
{
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Notification.RegistrationAddress);
	if (RemoteEndpoint.IsValid())
	{
		RemoteEndpoint->ForwardBusNotification(Notification.NotificationType);
	}

	if (Notification.NotificationType == EMessageBusNotification::Unregistered)
	{
		{
			FScopeLock RemoteAddressesLock(&RemoteAddressesCS);

			if (!RemoteAddresses.Contains(Notification.RegistrationAddress))
			{
				return;
			}
		}

		RemoveRemoteEndpoint(Notification.RegistrationAddress);
	}
}

void FConcertLocalEndpoint::ProcessEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FMessageAddress& InRemoteAddress)
{
	// TODO: potentially add something to disable the handling
	const FConcertEndpointDiscoveryEvent* Message = ConcertContext.GetMessage<FConcertEndpointDiscoveryEvent>();

	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Message->ConcertEndpointId);
	if (!RemoteEndpoint.IsValid())
	{
		FConcertRemoteEndpointRef NewRemoteEndpoint = CreateRemoteEndpoint(FConcertEndpointContext{ Message->ConcertEndpointId, EndpointContext.EndpointFriendlyName }, ConcertContext.UtcNow, InRemoteAddress);
		Logger.LogRemoteEndpointDiscovery(ConcertContext, EndpointContext.EndpointId);
		PendingRemoteEndpointConnectionChangedEvents.Add(MakeTuple(NewRemoteEndpoint->GetEndpointContext(), EConcertRemoteEndpointConnection::Discovered));

		// Negotiate a reliable channel
		if (MessageEndpoint.IsValid())
		{
			FConcertReliableHandshakeData* InitialHandshake = FMessageEndpoint::MakeMessage<FConcertReliableHandshakeData>();
			InitialHandshake->ConcertEndpointId = EndpointContext.EndpointId;
			InitialHandshake->MessageId = FGuid::NewGuid();
			InitialHandshake->EndpointTimeoutTick = FTimespan(0, 0, Settings.RemoteEndpointTimeoutSeconds).GetTicks();
			NewRemoteEndpoint->FillReliableHandshakeResponse(EConcertReliableHandshakeState::Negotiate, *InitialHandshake);

			Logger.LogSendReliableHandshake(*InitialHandshake, Message->ConcertEndpointId, ConcertContext.UtcNow);

			// Update the last sent message time to this endpoint
			NewRemoteEndpoint->SetLastSentMessageTime(ConcertContext.UtcNow);

			MessageEndpoint->Send(
				InitialHandshake, // Should be deleted by MessageBus
				EMessageFlags::Reliable,
				nullptr, // No Attachment
				TArrayBuilder<FMessageAddress>().Add(NewRemoteEndpoint->GetAddress()),
				FTimespan::Zero(), // No Delay
				FDateTime::MaxValue() // No Expiration
			);
		}
	}
	else
	{
		// We received a message from the remote endpoint, make sure we update our last receipt time. 
		RemoteEndpoint->UpdateLastMessageReceivedTime(ConcertContext.UtcNow);
	}
}

void FConcertLocalEndpoint::ProcessReliableHandshake(const FConcertMessageContext& ConcertContext)
{
	SCOPED_CONCERT_TRACE(FConcertLocalEndpoint_ProcessReliableHandshake);
	const FConcertReliableHandshakeData* Message = ConcertContext.GetMessage<FConcertReliableHandshakeData>();

	// This should always exist as FConcertReliableHandshakeData is also a FConcertEndpointDiscoveryEvent message, so should have added the endpoint in ProcessEndpointDiscovery
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Message->ConcertEndpointId);
	check(RemoteEndpoint.IsValid());

	Logger.LogReceiveReliableHandshake(*Message, EndpointContext.EndpointId, ConcertContext.UtcNow);

	FConcertReliableHandshakeData* HandshakeResponse = FMessageEndpoint::MakeMessage<FConcertReliableHandshakeData>();
	HandshakeResponse->ConcertEndpointId = EndpointContext.EndpointId;
	HandshakeResponse->MessageId = FGuid::NewGuid();
	HandshakeResponse->EndpointTimeoutTick = FTimespan(0, 0, Settings.RemoteEndpointTimeoutSeconds).GetTicks();
	if (!RemoteEndpoint->HandleReliableHandshake(*Message, *HandshakeResponse) || !MessageEndpoint.IsValid())
	{
		HandshakeResponse->~FConcertReliableHandshakeData();
		FMemory::Free(HandshakeResponse);
		return;
	}

	Logger.LogSendReliableHandshake(*HandshakeResponse, Message->ConcertEndpointId, ConcertContext.UtcNow);

	// Update the last sent message time to this endpoint
	RemoteEndpoint->SetLastSentMessageTime(ConcertContext.UtcNow);

	MessageEndpoint->Send(
		HandshakeResponse, // Should be deleted by MessageBus
		EMessageFlags::Reliable,
		nullptr, // No Attachment
		TArrayBuilder<FMessageAddress>().Add(RemoteEndpoint->GetAddress()),
		FTimespan::Zero(), // No Delay
		FDateTime::MaxValue() // No Expiration
	);

	// (Re)send any pending reliable messages
	SendPendingMessages(RemoteEndpoint.ToSharedRef(), ConcertContext.UtcNow);
}

void FConcertLocalEndpoint::HandleMessage(const FConcertMessageContext& ConcertContext)
{
	checkf(!bIsHandlingMessage, TEXT("Re-entrant call to HandleMessage!"));
	TGuardValue<bool> SetIsHandlingMessage(bIsHandlingMessage, true);

	// Queue an acknowledgment for reliable messages
	QueueAck(ConcertContext);

	if (ConcertContext.MessageType->IsChildOf(FConcertEventData::StaticStruct()))
	{
		ProcessEvent(ConcertContext);
	}
	else if (ConcertContext.MessageType->IsChildOf(FConcertRequestData::StaticStruct()))
	{
		ProcessRequest(ConcertContext);
	}
	else if (ConcertContext.MessageType->IsChildOf(FConcertResponseData::StaticStruct()))
	{
		ProcessResponse(ConcertContext);
	}
	else if (ConcertContext.MessageType->IsChildOf(FConcertAckData::StaticStruct()))
	{
		ProcessAck(ConcertContext);
	}
}

void FConcertLocalEndpoint::ProcessEvent(const FConcertMessageContext& ConcertContext)
{
	const FConcertEventData* Event = ConcertContext.GetMessage<FConcertEventData>();
	const FTopLevelAssetPath EventType = ConcertContext.MessageType->GetStructPathName();

	Logger.LogProcessEvent(ConcertContext, EndpointContext.EndpointId);
	TSharedPtr<IConcertEventHandler> Handler = EventHandlers.FindRef(EventType);
	if (Handler.IsValid())
	{
		// TODO: do we want to allow async handling
		Handler->HandleEvent(ConcertContext);
	}
	else
	{
		// TODO: not handling this event error or fallback
	}
}

void FConcertLocalEndpoint::ProcessRequest(const FConcertMessageContext& ConcertContext)
{
	const FConcertRequestData* Request = ConcertContext.GetMessage<FConcertRequestData>();
	const FTopLevelAssetPath RequestType = ConcertContext.MessageType->GetStructPathName();

	// The response ID should match the request message, and the response should go back to the endpoint where the request came from
	auto DispatchResponse = [this, RequestMessageId = ConcertContext.Message->MessageId, ResponseDestinationEndpointId = Request->ConcertEndpointId](TSharedPtr<IConcertResponse> Response)
	{
		// if we didn't generate a response we had no handler for the request send generic unknown request response
		if (!Response.IsValid())
		{
			Response = MakeShared<TConcertResponse<FConcertResponseData>>(FConcertResponseData(EConcertResponseCode::UnknownRequest));
		}

		SetResponseSendingInfo(Response.ToSharedRef(), RequestMessageId);
		InternalQueueResponse(Response.ToSharedRef(), ResponseDestinationEndpointId);
	};

	Logger.LogProcessRequest(ConcertContext, EndpointContext.EndpointId);
	TSharedPtr<IConcertRequestHandler> Handler = RequestHandlers.FindRef(RequestType);
	if (Handler.IsValid())
	{
		Handler->HandleRequest(ConcertContext).Next(MoveTemp(DispatchResponse));
	}
	else
	{
		DispatchResponse(nullptr);
	}
}

void FConcertLocalEndpoint::ProcessResponse(const FConcertMessageContext& ConcertContext)
{
	// Get the Response
	const FConcertResponseData* Response = ConcertContext.GetMessage<FConcertResponseData>();

	Logger.LogProcessResponse(ConcertContext, EndpointContext.EndpointId);

	// Get the remote endpoint 
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Response->ConcertEndpointId);
	if (RemoteEndpoint.IsValid())
	{
		RemoteEndpoint->HandleResponse(ConcertContext);
	}
}

void FConcertLocalEndpoint::ProcessAck(const FConcertMessageContext& ConcertContext)
{
	Logger.LogProcessAck(ConcertContext, EndpointContext.EndpointId);

	// Get the remote endpoint 
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(ConcertContext.SenderConcertEndpointId);
	if (RemoteEndpoint.IsValid())
	{
		RemoteEndpoint->HandleAcknowledgement(ConcertContext);
	}
}

void FConcertLocalEndpoint::QueueReceivedMessage(const FConcertMessageContext& ConcertContext)
{
	const FConcertMessageData* Message = ConcertContext.GetMessage<FConcertMessageData>();

	// If we are receiving a message from an unknown endpoint, discard the message
	FConcertRemoteEndpointPtr RemoteEndpoint = FindRemoteEndpoint(Message->ConcertEndpointId);
	if (!RemoteEndpoint.IsValid())
	{
		Logger.LogMessageDiscarded(ConcertContext, EndpointContext.EndpointId, IConcertTransportLogger::EMessageDiscardedReason::UnknownEndpoint);
		return;
	}

	// Queue the message for handling on the next Tick
	RemoteEndpoint->QueueMessageToReceive(ConcertContext);
}

void FConcertLocalEndpoint::SendPendingMessages(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow)
{
	for (const TSharedPtr<IConcertMessage>& PendingMessage : RemoteEndpoint->GetPendingMessages())
	{
		if (PendingMessage->GetState() == EConcertMessageState::Pending)
		{
			SendMessage(PendingMessage.ToSharedRef(), RemoteEndpoint, UtcNow);
		}
	}
	RemoteEndpoint->ClearPendingResend();
}

void FConcertLocalEndpoint::SendKeepAlive(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	FConcertKeepAlive* KeepAlive = new FConcertKeepAlive();
	KeepAlive->ConcertEndpointId = EndpointContext.EndpointId;
	KeepAlive->MessageId = FGuid::NewGuid();

	// Update the last sent message time to this endpoint
	RemoteEndpoint->SetLastSentMessageTime(UtcNow);

	if (RemoteEndpoint->IsRegistered())
	{
		MessageEndpoint->Send<FConcertKeepAlive>(KeepAlive, RemoteEndpoint->GetAddress());
	}
	// if the remote endpoint isn't registered to the bus anymore, publish to it so it can be re-registered
	else
	{
		MessageEndpoint->Publish<FConcertKeepAlive>(KeepAlive);
	}
}

void FConcertLocalEndpoint::SendKeepAlives(const FDateTime& UtcNow)
{
	SCOPED_CONCERT_TRACE(FConcertLocalEndpoint_SendKeepAlives);
	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);

	for (const auto& RemoteEndpointPair : RemoteEndpoints)
	{
		FConcertRemoteEndpointPtr RemoteEndpoint = RemoteEndpointPair.Value;
		check(RemoteEndpoint.IsValid());

		// if no message have been sent to this endpoint for some time, send a keep alive to ensure the UDP nodes re-registered if MessageBus decided to unregister them.
		FTimespan KeepAliveSpan = RemoteEndpoint->GetEndpointTimeoutSpan().IsZero() ? FTimespan(0, 0, 10) : RemoteEndpoint->GetEndpointTimeoutSpan() * 0.25f;
		if (RemoteEndpoint->GetLastSentMessageTime() + KeepAliveSpan <= UtcNow)
		{
			SendKeepAlive(RemoteEndpoint.ToSharedRef(), UtcNow);
		}
	}
}

void FConcertLocalEndpoint::TimeoutRemoteEndpoints(const FDateTime& UtcNow)
{
	const FTimespan RemoteEndpointTimeoutSpan = FTimespan(0, 0, Settings.RemoteEndpointTimeoutSeconds);

	if (!RemoteEndpointTimeoutSpan.IsZero())
	{
		FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);

		for (auto It = RemoteEndpoints.CreateIterator(); It; ++It)
		{
			const FGuid EndpointId = It->Key;
			FConcertRemoteEndpointPtr RemoteEndpoint = It->Value;
			check(RemoteEndpoint.IsValid());
			if (RemoteEndpoint->GetLastReceivedMessageTime() + RemoteEndpointTimeoutSpan <= UtcNow)
			{
				PendingRemoteEndpointConnectionChangedEvents.Add(MakeTuple(RemoteEndpoint->GetEndpointContext(), EConcertRemoteEndpointConnection::TimedOut));
				Logger.LogRemoteEndpointTimeOut(EndpointId, UtcNow);
				SendEndpointClosed(RemoteEndpoint.ToSharedRef(), UtcNow);

				{
					FScopeLock RemoteAddressesLock(&RemoteAddressesCS);
					RemoteAddresses.Remove(RemoteEndpoint->GetAddress());
				}

				It.RemoveCurrent();
				continue;
			}
		}
	}
}

void FConcertLocalEndpoint::ProcessQueuedReceivedMessages(const FDateTime& UtcNow)
{
	if (bIsHandlingMessage)
	{
		return;
	}

	FScopeLock RemoteEndpointsLock(&RemoteEndpointsCS);

	for (auto It = RemoteEndpoints.CreateIterator(); It; ++It)
	{
		const FGuid EndpointId = It->Key;
		FConcertRemoteEndpointPtr RemoteEndpoint = It->Value;

		check(RemoteEndpoint.IsValid());

		bool bEndpointClosedRemotely = false;
		for (;;)
		{
			TSharedPtr<FConcertMessageCapturedContext> QueuedMessage = RemoteEndpoint->GetNextMessageToReceive(UtcNow);
			if (!QueuedMessage.IsValid())
			{
				break;
			}

			if (QueuedMessage->CapturedContext.MessageType->IsChildOf(FConcertEndpointClosedData::StaticStruct()))
			{
				bEndpointClosedRemotely = true;
				break;
			}

			HandleMessage(QueuedMessage->CapturedContext);
		}

		if (bEndpointClosedRemotely)
		{
			PendingRemoteEndpointConnectionChangedEvents.Add(MakeTuple(RemoteEndpoint->GetEndpointContext(), EConcertRemoteEndpointConnection::ClosedRemotely));
			Logger.LogRemoteEndpointClosure(EndpointId, UtcNow);
			It.RemoveCurrent();
			continue;
		}
	}
}

void FConcertLocalEndpoint::PurgeOldReceivedMessages(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow)
{
	const FTimespan PurgeProcessedMessageSpan = FTimespan(0, 0, Settings.PurgeProcessedMessageDelaySeconds);
	for (const auto& RemoteEndpoint : InRemoteEndpoints)
	{
		check(RemoteEndpoint.IsValid());
		RemoteEndpoint->PurgeOldReceivedMessages(UtcNow, PurgeProcessedMessageSpan);
	}
}

void FConcertLocalEndpoint::ResendPendingMessages(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow)
{
	for (const FConcertRemoteEndpointPtr& RemoteEndpoint : InRemoteEndpoints)
	{
		if (RemoteEndpoint->IsPendingResend())
		{
			SendPendingMessages(RemoteEndpoint.ToSharedRef(), UtcNow);
		}
	}
}
