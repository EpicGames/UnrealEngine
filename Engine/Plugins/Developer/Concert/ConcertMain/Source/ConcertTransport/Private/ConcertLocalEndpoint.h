// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertEndpoint.h"
#include "IConcertTransportLogger.h"
#include "ConcertRemoteEndpoint.h"
#include "ConcertTransportSettings.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"

class FMessageEndpoint;
struct FMessageBusNotification;

class FConcertLocalEndpointKeepAliveRunnable;

typedef TSharedPtr<FConcertRemoteEndpoint, ESPMode::ThreadSafe> FConcertRemoteEndpointPtr;
typedef TSharedRef<FConcertRemoteEndpoint, ESPMode::ThreadSafe> FConcertRemoteEndpointRef;

/**
 * Implements a local endpoint for Concert
 */
class FConcertLocalEndpoint
	: public IConcertLocalEndpoint
{
	friend class FConcertLocalEndpointKeepAliveRunnable;

public:
	FConcertLocalEndpoint(const FString& InEndpointFriendlyName, const FConcertEndpointSettings& InEndpointSettings, const FConcertTransportLoggerFactory& InLogFactory);
	virtual ~FConcertLocalEndpoint();

	virtual const FConcertEndpointContext& GetEndpointContext() const override;
	virtual TArray<FConcertEndpointContext> GetRemoteEndpoints() const override;
	virtual FMessageAddress GetRemoteAddress(const FGuid& ConcertEndpointId) const override;

	virtual FOnConcertRemoteEndpointConnectionChanged& OnRemoteEndpointConnectionChanged() override;
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() override;

protected:
	virtual void InternalAddRequestHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IConcertRequestHandler>& Handler) override;
	virtual void InternalAddEventHandler(const FTopLevelAssetPath& EventMessageType, const TSharedRef<IConcertEventHandler>& Handler) override;

	virtual void InternalRemoveRequestHandler(const FTopLevelAssetPath& RequestMessageType) override;
	virtual void InternalRemoveEventHandler(const FTopLevelAssetPath& EventMessageType) override;

	virtual void InternalSubscribeToEvent(const FTopLevelAssetPath& EventMessageType) override;
	virtual void InternalUnsubscribeFromEvent(const FTopLevelAssetPath& EventMessageType) override;

	virtual void InternalQueueRequest(const TSharedRef<IConcertRequest>& Request, const FGuid& Endpoint) override;
	virtual void InternalQueueResponse(const TSharedRef<IConcertResponse>& Response, const FGuid& Endpoint) override;
	virtual void InternalQueueEvent(const TSharedRef<IConcertEvent>& Event, const FGuid& Endpoint, EConcertMessageFlags Flags) override;
	virtual void InternalPublishEvent(const TSharedRef<IConcertEvent>& Event) override;

private:
	/** Create a new remote endpoint and add it to the RemoteEndpoints map */
	FConcertRemoteEndpointRef CreateRemoteEndpoint(const FConcertEndpointContext& InEndpointContext, const FDateTime& InLastReceivedMessageTime, const FMessageAddress& InRemoteAddress);

	/** Find the remote endpoint with the given ID */
	FConcertRemoteEndpointPtr FindRemoteEndpoint(const FGuid& InEndpointId) const;

	/** Find the remote endpoint with the given Message Address */
	FConcertRemoteEndpointPtr FindRemoteEndpoint(const FMessageAddress& InEndpointId) const;

	/** Remove the remote endpoint with the given Message Address. */
	void RemoveRemoteEndpoint(const FMessageAddress& EndpointAddress);

	/** Handle the endpoint ticker */
	bool HandleTick(float DeltaTime);

	/** Queue an acknowledgment to a specific message */
	void QueueAck(const FConcertMessageContext& ConcertContext);

	/** Send any pending acknowledgments */
	void SendAcks(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow);

	/** Send any pending acknowledgment to the remote endpoint */
	void SendAck(const FGuid& AcknowledgmentToSend, const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow);

	/** Send a notification to this endpoint that it has been closed on the remote peer (us!) */
	void SendEndpointClosed(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow);

	/** Publish a message */
	void PublishMessage(const TSharedRef<IConcertMessage>& Message);

	/** Send a message to a specific remote endpoint */
	void SendMessage(const TSharedRef<IConcertMessage>& Message, const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow, EConcertMessageFlags Flags = EConcertMessageFlags::None);

	/** Handle an incoming message from the message bus */
	void InternalHandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handle disconnection notifications coming from MessageBus */
	void InternalHandleBusNotification(const FMessageBusNotification& Notification);

	/** Process an endpoint discovery message to add a new known remote endpoint */
	void ProcessEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FMessageAddress& InRemoteAddress);

	/** Process a reliable handshake message for a known remote endpoint */
	void ProcessReliableHandshake(const FConcertMessageContext& ConcertContext);

	/** Handle a concert message*/
	void HandleMessage(const FConcertMessageContext& ConcertContext);

	/** Process a concert event using a registered event handler */
	void ProcessEvent(const FConcertMessageContext& ConcertContext);

	/** Process a concert request using a registered request handler */
	void ProcessRequest(const FConcertMessageContext& ConcertContext);

	/** Process a concert response from a remote endpoint */
	void ProcessResponse(const FConcertMessageContext& ConcertContext);

	/** Process an acknowledgment to a reliable event sent from this endpoint */
	void ProcessAck(const FConcertMessageContext& ConcertContext);

	/** Queue a received message for processing by ProcessQueuedReceivedMessages */
	void QueueReceivedMessage(const FConcertMessageContext& ConcertContext);

	/** Resend pending reliable messages */
	void SendPendingMessages(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow);

	/** Send a keep alive message the a remote endpoint */
	void SendKeepAlive(const FConcertRemoteEndpointRef& RemoteEndpoint, const FDateTime& UtcNow);

	/** Send keep alive messages to remote endpoint we haven't sent to in a while. */
	void SendKeepAlives(const FDateTime& UtcNow);

	/** Timeout remote endpoint from which we haven't received a message or keep alive in a while */
	void TimeoutRemoteEndpoints(const FDateTime& UtcNow);

	/** Process messages that have been received out of order */
	void ProcessQueuedReceivedMessages(const FDateTime& UtcNow);

	/** Purge old already received messages after a certain period of time */
	void PurgeOldReceivedMessages(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow);

	/** Resend pending messages to remote endpoints if need be. */
	void ResendPendingMessages(const TArray<FConcertRemoteEndpointPtr>& InRemoteEndpoints, const FDateTime& UtcNow);

	/** Handles the inbound message from MessageBus. Operates on the game thread. */
	void HandleInboundMessages(const FDateTime& UtcNow);

	/** Updates the last receive time for a KeepAlive message. */
	void ProcessKeepAliveMessage(const TSharedPtr<IMessageContext, ESPMode::ThreadSafe>& Context, const FDateTime& UtcNow);

	/** Force pending resend on endpoints */
	void ForcePendingResend();

	/** This context of this endpoint */
	FConcertEndpointContext EndpointContext;

	/** Next reliable channel ID to use when creating a remote endpoint */
	uint16 NextReliableChannelId;

	/** Map of remote Endpoints we are sending messages to from this endpoint */
	mutable FCriticalSection RemoteEndpointsCS;
	TMap<FGuid, FConcertRemoteEndpointPtr> RemoteEndpoints;

	/** Array of remote endpoint message addresses that were discovered. */
	mutable FCriticalSection RemoteAddressesCS;
	TArray<FMessageAddress> RemoteAddresses;

	/** Incoming message bus messages */
	TQueue<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>, EQueueMode::Spsc> InboundMessages;

	/** Callback when a remote endpoint connection status changes. */
	TArray<TTuple<FConcertEndpointContext, EConcertRemoteEndpointConnection>> PendingRemoteEndpointConnectionChangedEvents;
	FOnConcertRemoteEndpointConnectionChanged OnRemoteEndpointConnectionChangedDelegate;

	/** Callback when a message has been acknowledged by a remote endpoint */
	FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint OnConcertMessageAcknowledgementReceivedDelegate; 

	/** Registered message handlers that do not generate a response */
	TMap<FTopLevelAssetPath, TSharedPtr<IConcertEventHandler>> EventHandlers;

	/** Registered message handlers that returns a response */
	TMap<FTopLevelAssetPath, TSharedPtr<IConcertRequestHandler>> RequestHandlers;

	/** Handle to the registered ticker for request and reliable message */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Runnable thread used to send keep-alive messages even when the game thread is blocked */
	TUniquePtr<FConcertLocalEndpointKeepAliveRunnable> KeepAliveRunnable;

	/** True if we are currently handling a message (used to prevent re-entrant behavior if something pumps the task pool while we're handling a message) */
	bool bIsHandlingMessage;

	/** Holds the messaging endpoint we are sending from */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds the endpoint settings */
	FConcertEndpointSettings Settings;

	/** Holds the Transport Logger, if any. */
	FConcertTransportLoggerWrapper Logger;
};
