// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"

DEFINE_LOG_CATEGORY(LogConcert);
DEFINE_LOG_CATEGORY(LogConcertDebug);
