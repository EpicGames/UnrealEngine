// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertMessageContext.h"

FConcertMessageCapturedContext::FConcertMessageCapturedContext(const FConcertMessageContext& InContext)
	: CapturedContext()
{
	CapturedContext.SenderConcertEndpointId = InContext.SenderConcertEndpointId;
	CapturedContext.UtcNow = InContext.UtcNow;
	CapturedContext.Annotations = InContext.Annotations;
	MallocCapturedContextMessage(InContext.Message, InContext.MessageType);
}

FConcertMessageCapturedContext::~FConcertMessageCapturedContext()
{
	FreeCapturedContextMessage();
}

FConcertMessageCapturedContext::FConcertMessageCapturedContext(FConcertMessageCapturedContext&& Other)
	: CapturedContext()
{
	CapturedContext.SenderConcertEndpointId = Other.CapturedContext.SenderConcertEndpointId;
	CapturedContext.UtcNow = Other.CapturedContext.UtcNow;
	CapturedContext.Annotations = MoveTemp(Other.CapturedContext.Annotations);
	Exchange(CapturedContext.MessageType, Other.CapturedContext.MessageType); // Swap null with Other
	Exchange(CapturedContext.Message, Other.CapturedContext.Message); // Swap null with Other
}

const FConcertMessageCapturedContext& FConcertMessageCapturedContext::operator=(FConcertMessageCapturedContext&& Other)
{
	if (this != &Other)
	{
		FreeCapturedContextMessage();

		CapturedContext.SenderConcertEndpointId = Other.CapturedContext.SenderConcertEndpointId;
		CapturedContext.UtcNow = Other.CapturedContext.UtcNow;
		CapturedContext.Annotations = MoveTemp(Other.CapturedContext.Annotations);
		Exchange(CapturedContext.MessageType, Other.CapturedContext.MessageType); // Swap null with Other
		Exchange(CapturedContext.Message, Other.CapturedContext.Message); // Swap null with Other
	}
	return *this;
}

void FConcertMessageCapturedContext::MallocCapturedContextMessage(const FConcertMessageData* InMessage, const UScriptStruct* InMessageType)
{
	check(CapturedContext.Message == nullptr);

	// Allocate and copy the message from the original
	void* MessageData = FMemory::Malloc(InMessageType->GetStructureSize());
	InMessageType->InitializeStruct(MessageData);
	InMessageType->CopyScriptStruct(MessageData, InMessage);

	CapturedContext.MessageType = InMessageType;
	CapturedContext.Message = (FConcertMessageData*)MessageData;
}

void FConcertMessageCapturedContext::FreeCapturedContextMessage()
{
	// Deallocate the message we copied
	void* MessageData = (void*)CapturedContext.Message;
	if (MessageData)
	{
		FMemory::Free(MessageData);
	}

	CapturedContext.MessageType = nullptr;
	CapturedContext.Message = nullptr;
}
