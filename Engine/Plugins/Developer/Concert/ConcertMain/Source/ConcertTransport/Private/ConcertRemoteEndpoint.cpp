// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertRemoteEndpoint.h"
#include "ConcertLogGlobal.h"
#include "IMessageBusListener.h"

FConcertRemoteEndpoint::FConcertRemoteEndpoint(const FConcertEndpointContext& InEndpointContext, const uint16 InReliableChannelId, const FDateTime& InLastReceivedMessageTime, const FTimespan& InEndpointTimeoutSpan, const FMessageAddress& InAddress, const IConcertTransportLoggerPtr& InLogger)
	: EndpointContext(InEndpointContext)
	, ReliableChannelIdToSend(InReliableChannelId)
	, ReliableChannelIdToReceive(FConcertMessageData::UnreliableChannelId)
	, LastAckTime(0)
	, EndpointTimeoutSpan(InEndpointTimeoutSpan)
	, LastReceivedMessageTime(InLastReceivedMessageTime)
	, LastSentMessageTime(0)
	, NextMessageIndexToSend(0)
	, NextMessageIndexToReceive(0)
	, NextAcknowledgmentToSend()
	, bNeedResendPendingMessages(false)
	, bIsRegistered(true)
	, Address(InAddress)
	, Logger(InLogger)
{
}

FConcertRemoteEndpoint::~FConcertRemoteEndpoint()
{
	// Timeout all  left over messages not to leave any unfulfilled promises
	TimeoutAllMessages();
}

const FConcertEndpointContext& FConcertRemoteEndpoint::GetEndpointContext() const
{
	return EndpointContext;
}

FOnConcertMessageAcknowledgementReceived& FConcertRemoteEndpoint::OnConcertMessageAcknowledgementReceived()
{
	return OnConcertMessageAcknowledgementReceivedDelegate;
}

bool FConcertRemoteEndpoint::HasReliableChannel() const
{
	return ReliableChannelIdToReceive != FConcertMessageData::UnreliableChannelId;
}

bool FConcertRemoteEndpoint::IsRegistered() const
{
	return bIsRegistered;
}

bool FConcertRemoteEndpoint::IsPendingResend() const
{
	return bNeedResendPendingMessages;
}

void FConcertRemoteEndpoint::ClearPendingResend()
{
	bNeedResendPendingMessages = false;
}

void FConcertRemoteEndpoint::QueueMessageToSend(const TSharedRef<IConcertMessage>& Message)
{
	// Assign a message index to allow ordering
	SetMessageOrderIndex(Message, NextMessageIndexToSend++);
	SetMessageChannelId(Message, ReliableChannelIdToSend);

	PendingMessages.Add(Message);
}

void FConcertRemoteEndpoint::QueueMessageToReceive(const FConcertMessageContext& Context)
{
	UpdateLastMessageReceivedTime(Context.UtcNow);

	// Keep alive messages only need to update LastReceivedMessageTime so we can discard those now
	if (Context.MessageType->IsChildOf(FConcertKeepAlive::StaticStruct()))
	{
		return;
	}

	// If the message was already received then discard it
	if (Context.Message->IsReliable())
	{
		FDateTime* LastReceivedTime = RecentlyReceivedMessages.Find(Context.Message->MessageId);
		if (LastReceivedTime)
		{
			*LastReceivedTime = Context.UtcNow; // Update the last received time
			Logger.LogMessageDiscarded(Context, EndpointContext.EndpointId, IConcertTransportLogger::EMessageDiscardedReason::AlreadyProcessed);
			return;
		}

		// Do not process messages multiple time
		RecentlyReceivedMessages.Add(Context.Message->MessageId, Context.UtcNow);
	}

	// Queue all messages so we can process them safely on the next Tick
	QueuedMessagesToReceive.Add(MakeShared<FConcertMessageCapturedContext>(Context));
}

bool FConcertRemoteEndpoint::HandleReliableHandshake(const FConcertReliableHandshakeData& InHandshakeRequest, FConcertReliableHandshakeData& OutHandshakeResponse)
{
	if (ReliableChannelIdToReceive != InHandshakeRequest.ReliableChannelId)
	{
		// Process the handshake and update our state
		NextMessageIndexToReceive = InHandshakeRequest.NextMessageIndex;
		ReliableChannelIdToReceive = InHandshakeRequest.ReliableChannelId;
		EndpointTimeoutSpan = FTimespan(InHandshakeRequest.EndpointTimeoutTick);
		RecentlyReceivedMessages.Reset();
		NextAcknowledgmentToSend.Reset();
		LastAckTime = FDateTime::UtcNow();

		// Update the channel ID on any pending messages and trim any partially completed messages
		for (auto It = PendingMessages.CreateIterator(); It; ++It)
		{
			TSharedRef<IConcertMessage> PendingMessage = It->ToSharedRef();
			if (PendingMessage->GetState() != EConcertMessageState::Pending)
			{
				It.RemoveCurrent();
				continue;
			}
			SetMessageChannelId(PendingMessage, ReliableChannelIdToSend);
		}

		// Reset the order indices of the remaining pending messages
		{
			int32 PendingMessageSendIndex = NextMessageIndexToSend - 1;
			for (int32 PendingMessageIndex = PendingMessages.Num() - 1; PendingMessageIndex >= 0; --PendingMessageIndex)
			{
				TSharedRef<IConcertMessage> PendingMessage = PendingMessages[PendingMessageIndex].ToSharedRef();
				SetMessageOrderIndex(PendingMessage, PendingMessageSendIndex--);
			}
		}

		// Clear out any queued received messages for anything but our new channel
		for (auto It = QueuedOutOfOrderMessagesToReceive.CreateIterator(); It; ++It)
		{
			if (It.Key().ChannelId != ReliableChannelIdToReceive)
			{
				It.RemoveCurrent();
				continue;
			}
		}

		// Respond that the negotiation was successful
		FillReliableHandshakeResponse(EConcertReliableHandshakeState::Success, OutHandshakeResponse);
		return true;
	}

	return false;
}

void FConcertRemoteEndpoint::FillReliableHandshakeResponse(const EConcertReliableHandshakeState InResponseState, FConcertReliableHandshakeData& OutHandshakeResponse) const
{
	OutHandshakeResponse.HandshakeState = InResponseState;
	OutHandshakeResponse.ReliableChannelId = ReliableChannelIdToSend;
	OutHandshakeResponse.NextMessageIndex = NextMessageIndexToSend;

	// Count the number of pending messages that we'll re-send after negotiating the reliable channel, 
	// as these will affect the NextMessageIndex we send
	{
		int32 NumPendingMessages = 0;
		for (const TSharedPtr<IConcertMessage>& PendingMessage : PendingMessages)
		{
			if (PendingMessage->GetState() == EConcertMessageState::Pending)
			{
				++NumPendingMessages;
			}
		}
		OutHandshakeResponse.NextMessageIndex -= NumPendingMessages;
	}
}

void FConcertRemoteEndpoint::QueueAcknowledgmentToSend(const FGuid& MessageId)
{
	NextAcknowledgmentToSend = MessageId;
}

TOptional<FGuid> FConcertRemoteEndpoint::GetNextAcknowledgmentToSend()
{
	TOptional<FGuid> NextAcknowledgmentToSendCopy = NextAcknowledgmentToSend;
	NextAcknowledgmentToSend.Reset();
	return NextAcknowledgmentToSendCopy;
}

void FConcertRemoteEndpoint::HandleAcknowledgement(const FConcertMessageContext& Context)
{
	const FConcertAckData* Message = Context.GetMessage<FConcertAckData>();

	// Ignore old acknowledgments
	const FDateTime AckSendTime = Message->AckSendTimeTicks;
	if (AckSendTime <= LastAckTime)
	{
		return;
	}
	LastAckTime = AckSendTime;

	// Acknowledge everything up-to and including the message ID of this acknowledgment
	bool bFoundMessageToAck = false;
	for (auto It = PendingMessages.CreateIterator(); It; ++It)
	{
		TSharedRef<IConcertMessage> PendingMessage = It->ToSharedRef();

		if (PendingMessage->GetState() == EConcertMessageState::Pending)
		{
			PendingMessage->Acknowledge(Context);
			OnConcertMessageAcknowledgementReceivedDelegate.Broadcast(GetEndpointContext(), PendingMessage, Context);
		}
		if (PendingMessage->GetState() == EConcertMessageState::Completed)
		{
			// Message is fully acknowledged - remove it now
			It.RemoveCurrent();
		}

		if (PendingMessage->GetMessageId() == Message->SourceMessageId)
		{
			// Processed everything we should have for this acknowledgment
			bFoundMessageToAck = true;
			break;
		}
	}
	UE_CLOGF(!bFoundMessageToAck, LogConcert, Warning, "%ls: Failed to find message '%ls' to acknowledge.", *EndpointContext.ToString(), *Message->SourceMessageId.ToString());
}

void FConcertRemoteEndpoint::HandleResponse(const FConcertMessageContext& Context)
{
	const FConcertResponseData* Message = Context.GetMessage<FConcertResponseData>();

	// Find the associated request
	for (TSharedPtr<IConcertMessage> PendingMessage : PendingMessages)
	{
		if (PendingMessage->GetMessageId() == Message->RequestMessageId)
		{
			PendingMessage->Acknowledge(Context);
			check(PendingMessage->GetState() == EConcertMessageState::Completed);
			break;
		}
	}
}

void FConcertRemoteEndpoint::ForwardBusNotification(EMessageBusNotification Notification)
{
	bNeedResendPendingMessages |= !bIsRegistered && Notification == EMessageBusNotification::Registered;
	bIsRegistered = Notification == EMessageBusNotification::Registered;
}

void FConcertRemoteEndpoint::PurgeOldReceivedMessages(const FDateTime& UtcNow, const FTimespan& PurgeProcessedMessageSpan)
{
	for (auto It = RecentlyReceivedMessages.CreateIterator(); It; ++It)
	{
		if (It.Value() + PurgeProcessedMessageSpan <= UtcNow)
		{
			It.RemoveCurrent();
			continue;
		}
	}
}

TSharedPtr<FConcertMessageCapturedContext> FConcertRemoteEndpoint::GetNextMessageToReceive(const FDateTime& UtcNow)
{
	// Process each pending message, potentially re-queuing reliable messages that arrived out-of-order
	{
		TSharedPtr<FConcertMessageCapturedContext> NextMessageToReceive;
		while (QueuedMessagesToReceive.Num() > 0 && !NextMessageToReceive.IsValid())
		{
			// Pop the first message off the queue
			NextMessageToReceive = QueuedMessagesToReceive[0];
			QueuedMessagesToReceive.RemoveAt(0, EAllowShrinking::No);

			NextMessageToReceive->CapturedContext.UtcNow = UtcNow;

			const bool bIsSafeToHandle = NextMessageToReceive->CapturedContext.Message->IsSafeToHandle();
			const bool bIsReliable = NextMessageToReceive->CapturedContext.Message->IsReliable();
			const bool bIsValidChannel = !bIsReliable || NextMessageToReceive->CapturedContext.Message->ChannelId == ReliableChannelIdToReceive;
			const bool bCanProcessMessage = bIsSafeToHandle && bIsValidChannel && (!bIsReliable || NextMessageToReceive->CapturedContext.Message->MessageOrderIndex == NextMessageIndexToReceive);

			// Attempt to process this message now if it is possible to do so
			// If not, either discard the message (if unreliable), or queue it for processing later (if reliable)
			if (bCanProcessMessage)
			{
				// Update the reliable message index to receive
				if (bIsReliable && NextMessageToReceive->CapturedContext.Message->MessageOrderIndex == NextMessageIndexToReceive)
				{
					++NextMessageIndexToReceive;
				}
			}
			else
			{
				// Re-queue the reliable message and we'll try and process it again later
				if (bIsReliable)
				{
					QueuedOutOfOrderMessagesToReceive.Add(FChannelIdAndOrderIndex{ NextMessageToReceive->CapturedContext.Message->ChannelId, NextMessageToReceive->CapturedContext.Message->MessageOrderIndex }, NextMessageToReceive);
					Logger.LogMessageQueued(NextMessageToReceive->CapturedContext, EndpointContext.EndpointId);
				}

				// Empty the pointer and try the next message
				NextMessageToReceive.Reset();
			}
		}
		if (NextMessageToReceive.IsValid())
		{
			return NextMessageToReceive;
		}
	}

	// Process the next reliable message that we've received
	{
		TSharedPtr<FConcertMessageCapturedContext> NextMessageToReceive;
		if (QueuedOutOfOrderMessagesToReceive.RemoveAndCopyValue(FChannelIdAndOrderIndex{ ReliableChannelIdToReceive, NextMessageIndexToReceive }, NextMessageToReceive))
		{
			NextMessageToReceive->CapturedContext.UtcNow = UtcNow;

			const bool bIsSafeToHandle = NextMessageToReceive->CapturedContext.Message->IsSafeToHandle();

			// Re-queue this message if it still can't be processed yet
			if (bIsSafeToHandle)
			{
				++NextMessageIndexToReceive;
			}
			else
			{
				QueuedOutOfOrderMessagesToReceive.Add(FChannelIdAndOrderIndex{ ReliableChannelIdToReceive, NextMessageIndexToReceive }, NextMessageToReceive);
				NextMessageToReceive.Reset();
			}
		}
		if (NextMessageToReceive.IsValid())
		{
			return NextMessageToReceive;
		}
	}

	return nullptr;
}

void FConcertRemoteEndpoint::TimeoutAllMessages()
{
	const FDateTime UtcNow = FDateTime::UtcNow();
	for (const TSharedPtr<IConcertMessage>& PendingMessage : PendingMessages)
	{
		Logger.LogTimeOut(PendingMessage.ToSharedRef(), EndpointContext.EndpointId, UtcNow);
		PendingMessage->TimeOut();
	}
	PendingMessages.Reset();
	LastAckTime = FDateTime::UtcNow();
}
