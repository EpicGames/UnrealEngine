// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "IConcertEndpoint.h"
#include "IMessageContext.h"
#include "IConcertTransportLogger.h"
#include "Containers/SortedMap.h"

enum class EMessageBusNotification : uint8;
class FConcertLocalEndpoint;

class FConcertRemoteEndpoint : public IConcertRemoteEndpoint
{
public:
	FConcertRemoteEndpoint(const FConcertEndpointContext& InEndpointContext, const uint16 InReliableChannelId, const FDateTime& InLastReceivedMessageTime, const FTimespan& InEndpointTimeoutSpan, const FMessageAddress& InAddress, const IConcertTransportLoggerPtr& InLogger);

	virtual ~FConcertRemoteEndpoint();

	virtual const FConcertEndpointContext& GetEndpointContext() const override;

	virtual FOnConcertMessageAcknowledgementReceived& OnConcertMessageAcknowledgementReceived() override;

	/** Does this remote endpoint have a reliable channel? */
	bool HasReliableChannel() const;

	/** Is the remote endpoint registered in the underlying bus. */
	bool IsRegistered() const;

	/** Does this remote endpoint needs to resend any of its pending messages. */
	bool IsPendingResend() const;

	/** Clear the pending resend flag. */
	void ClearPendingResend();
	
	/**
	 * Queue a message to send to this endpoint reliably
	 */
	void QueueMessageToSend(const TSharedRef<IConcertMessage>& Message);

	/**
	 * Queue a message to receive from this endpoint
	 */
	void QueueMessageToReceive(const FConcertMessageContext& Context);

	/** Handle a reliable handshake message for a known remote endpoint */
	bool HandleReliableHandshake(const FConcertReliableHandshakeData& InHandshakeRequest, FConcertReliableHandshakeData& OutHandshakeResponse);

	/** Fill a reliable handshake response message based on the current state of this endpoint */
	void FillReliableHandshakeResponse(const EConcertReliableHandshakeState InResponseState, FConcertReliableHandshakeData& OutHandshakeResponse) const;

	/**
	 * Queue an acknowledgment to send for the given message id
	 */
	void QueueAcknowledgmentToSend(const FGuid& MessageId);

	/**
	 * Get the pending acknowledgment to send on this endpoint, and reset the pending request
	 */
	TOptional<FGuid> GetNextAcknowledgmentToSend();

	/**
	 * Handle an acknowledgment received from this remote endpoint
	 */
	void HandleAcknowledgement(const FConcertMessageContext& Context);

	/**
	 * Handle a response to a request made to this remote endpoint
	 */
	void HandleResponse(const FConcertMessageContext& Context);

	/** Pass MessageBus notification to the endpoint */
	void ForwardBusNotification(EMessageBusNotification Notification);

	/** Purge old already received messages after a certain period of time */
	void PurgeOldReceivedMessages(const FDateTime& UtcNow, const FTimespan& PurgeProcessedMessageSpan);

	/** Get the Timespan before the remote end point consider us timed out */
	FTimespan GetEndpointTimeoutSpan() const { return EndpointTimeoutSpan; }

	void UpdateLastMessageReceivedTime(const FDateTime& Now)
	{
		LastReceivedMessageTime = FGenericPlatformMath::Max<FDateTime>(LastReceivedMessageTime, Now);
	}

	/** Get the time of last message received from this endpoint. */
	FDateTime GetLastReceivedMessageTime() const { return LastReceivedMessageTime; }

	/** Get the time of last message sent to this endpoint.*/
	FDateTime GetLastSentMessageTime() const { return LastSentMessageTime; }

	/** Set the time of last message sent to this endpoint */
	void SetLastSentMessageTime(const FDateTime& SendTime) { LastSentMessageTime = SendTime; }

	/** Get the remote endpoint MessageBus address. */
	const FMessageAddress& GetAddress() const { return Address; }

	/** Get the list of yet to be acknowledged messages sent to this endpoint. */
	const TArray<TSharedPtr<IConcertMessage>>& GetPendingMessages() const { return PendingMessages; }

	/** Get the next message to handle from the queued list, if any. */
	TSharedPtr<FConcertMessageCapturedContext> GetNextMessageToReceive(const FDateTime& UtcNow);

	/** Forces the resend pending messages flag to be set for this endpoint. This if for debugging use only. */
	void MarkForResend()
	{
		bNeedResendPendingMessages = true;
	}

private:
	/** */
	void TimeoutAllMessages();

	struct FChannelIdAndOrderIndex
	{
		uint16 ChannelId;
		uint16 OrderIndex;

		bool operator==(const FChannelIdAndOrderIndex& Other) const
		{
			return ChannelId == Other.ChannelId
				&& OrderIndex == Other.OrderIndex;
		}

		friend inline uint32 GetTypeHash(const FChannelIdAndOrderIndex& Key)
		{
			return HashCombine(GetTypeHash(Key.ChannelId), GetTypeHash(Key.OrderIndex));
		}
	};

	/** Array of messages that we haven't received a response or acknowledgment for yet in the order they were sent */
	TArray<TSharedPtr<IConcertMessage>> PendingMessages;

	/** Map of messages we recently received (message id -> time received) */
	TMap<FGuid, FDateTime> RecentlyReceivedMessages;

	/** Map of reliable messages that are pending receipt because they arrived out-of-order ((channel id + message order index) -> data) */
	TMap<FChannelIdAndOrderIndex, TSharedPtr<FConcertMessageCapturedContext>> QueuedOutOfOrderMessagesToReceive;

	/** Array of messages that are pending receipt */
	TArray<TSharedPtr<FConcertMessageCapturedContext>> QueuedMessagesToReceive;

	/** This context of this endpoint */
	FConcertEndpointContext EndpointContext;

	/** Callback when a message has been acknowledged by this remote endpoint */
	FOnConcertMessageAcknowledgementReceived OnConcertMessageAcknowledgementReceivedDelegate;

	/** The reliable channel ID to send with reliable messages */
	uint16 ReliableChannelIdToSend;

	/** The reliable channel ID to receive with reliable messages (default is unreliable, correct value is set during reliable negotiation) */
	uint16 ReliableChannelIdToReceive;

	/** Time of the last acknowledgment we processed */
	FDateTime LastAckTime;

	/** Time before the Endpoint consider us timed out */
	TAtomic<FTimespan> EndpointTimeoutSpan;

	/** Time we last received a message or keep alive from this endpoint */
	TAtomic<FDateTime> LastReceivedMessageTime;

	/** Time we last sent a message or keep alive to this endpoint */
	TAtomic<FDateTime> LastSentMessageTime;

	/** The next message index to use when sending to this endpoint */
	TAtomic<uint16> NextMessageIndexToSend;

	/** The next message index we should process when receiving from this endpoint */
	uint16 NextMessageIndexToReceive;

	/** Next message ID to acknowledge on this endpoint (if any) */
	TOptional<FGuid> NextAcknowledgmentToSend;

	/** Flag indicating if pending messages need to be resent. */
	bool bNeedResendPendingMessages;

	/** Flag indicating if the endpoint is still registered on the underlying bus. */
	bool bIsRegistered;

	/** Remote Endpoint Address */
	FMessageAddress Address;

	/** Holds the Transport Logger, if any */
	FConcertTransportLoggerWrapper Logger;
};
