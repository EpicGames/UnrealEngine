// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertTransportModule.h"
#include "ConcertLocalEndpoint.h"

/**
* Implements the endpoint provider
*/
class FConcertEndpointProvider : public IConcertEndpointProvider
{
public:
	virtual TSharedPtr<IConcertLocalEndpoint> CreateLocalEndpoint(const FString& InEndpointFriendlyName, const FConcertEndpointSettings& InEndpointSettings, const FConcertTransportLoggerFactory& InLogFactory) const override
	{
		return MakeShared<FConcertLocalEndpoint>(InEndpointFriendlyName, InEndpointSettings, InLogFactory);
	}
};

/** Implement the ConcertTransport module interface */
class FConcertTransportModule : public IConcertTransportModule
{
public:
	virtual TSharedPtr<IConcertEndpointProvider> CreateEndpointProvider() const override
	{
		LLM_SCOPE_BYNAME(TEXT("Concert/ConcertTransport"));
		return MakeShared<FConcertEndpointProvider>();
	}
};

IMPLEMENT_MODULE(FConcertTransportModule, ConcertTransport);
