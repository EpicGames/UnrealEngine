// Copyright Epic Games, Inc. All Rights Reserved.

#include "IdentifierTable/ConcertIdentifierTable.h"

FConcertLocalIdentifierTable* FConcertLocalIdentifierTable::ForceFNameTableIndex = (FConcertLocalIdentifierTable*)INDEX_NONE;

FConcertLocalIdentifierTable::FConcertLocalIdentifierTable(const FConcertLocalIdentifierState& InState)
{
	SetState(InState);
}

bool FConcertLocalIdentifierTable::HasMappings() const
{
	return MappedNames.Num() > 0;
}

int32 FConcertLocalIdentifierTable::MapName(const FName& InName)
{
	if (const int32* ExistingIndex = NameToMappedIndex.Find(InName))
	{
		return *ExistingIndex;
	}

	const int32 NewIndex = MappedNames.Num();
	FName& MappedName = MappedNames.Add_GetRef(InName);
	MappedName.SetNumber(NAME_NO_NUMBER_INTERNAL); // Clear the number from the version we store
	return NameToMappedIndex.Add(MappedName, NewIndex);
}

bool FConcertLocalIdentifierTable::UnmapName(const int32 InIndex, FName& OutName) const
{
	const bool bIsMapped = MappedNames.IsValidIndex(InIndex);
	OutName = bIsMapped ? MappedNames[InIndex] : FName();
	return bIsMapped;
}

bool FConcertLocalIdentifierTable::HasName(const int32 InIndex) const
{
	return MappedNames.IsValidIndex(InIndex);
}

bool FConcertLocalIdentifierTable::HasName(const FName& InName, int32* OutIndexPtr) const
{
	if (const int32* ExistingIndex = NameToMappedIndex.Find(InName))
	{
		if (OutIndexPtr)
		{
			*OutIndexPtr = *ExistingIndex;
		}
		return true;
	}
	
	if (OutIndexPtr)
	{
		*OutIndexPtr = INDEX_NONE;
	}
	return false;
}

void FConcertLocalIdentifierTable::SetState(const FConcertLocalIdentifierState& InState)
{
	MappedNames.Reset();
	NameToMappedIndex.Reset();

	MappedNames.Reserve(InState.MappedNames.Num());
	NameToMappedIndex.Reserve(InState.MappedNames.Num());
	for (const FString& MappedNameStr : InState.MappedNames)
	{
		const FName MappedName = FName(*MappedNameStr, NAME_NO_NUMBER_INTERNAL, /*bSplitName*/false);
		const int32 NewIndex = MappedNames.Num();
		MappedNames.Add(MappedName);
		NameToMappedIndex.Add(MappedName, NewIndex);
	}
}

void FConcertLocalIdentifierTable::GetState(FConcertLocalIdentifierState& OutState) const
{
	OutState.MappedNames.Reserve(MappedNames.Num());
	for (const FName& MappedName : MappedNames)
	{
		OutState.MappedNames.Add(MappedName.GetPlainNameString());
	}
}
