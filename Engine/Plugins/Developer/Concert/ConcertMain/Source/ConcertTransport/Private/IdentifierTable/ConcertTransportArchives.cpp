// Copyright Epic Games, Inc. All Rights Reserved.

#include "IdentifierTable/ConcertTransportArchives.h"
#include "IdentifierTable/ConcertIdentifierTable.h"

namespace UE::Concert::Private::ConcertIdentifierArchiveUtil
{

enum class EConcertIdentifierSource : uint8
{
	/** Plain string value (no suffix) */
	PlainString,
	/** Hardcoded FName index value (see MAX_NETWORKED_HARDCODED_NAME) */
	HardcodedIndex_PackedInt,
	/** Local identifier table index value (see FConcertLocalIdentifierTable) */
	LocalIdentifierTableIndex_PackedInt,
	/** Local identifier table index value (see FConcertLocalIdentifierTable) */
	LocalIdentifierTableIndex_FixedSizeInt32,
	/** FName table index value (non-portable!) */
	FNameTableIndex_FixedSizeUInt32,
	/** Hardcoded FName index value (see MAX_NETWORKED_HARDCODED_NAME) */
	HardcodedIndex_FixedSizeInt32,
};

void WriteName(FArchive& Ar, const FName& Name, FConcertLocalIdentifierTable* LocalIdentifierTable)
{
	auto SerializeConcertIdentifierSource = [&Ar](EConcertIdentifierSource InSource)
	{
		Ar.Serialize(&InSource, sizeof(EConcertIdentifierSource));
	};

	auto SerializeIndexValue = [&Ar](int32 InIndex)
	{
		check(InIndex >= 0);
		uint32 UnsignedIndex = (uint32)InIndex;
		Ar.SerializeIntPacked(UnsignedIndex);
	};

	bool bSerializeNumberAsPacked = true;
	if (const EName* Ename = Name.ToEName(); Ename && ShouldReplicateAsInteger(*Ename, Name))
	{
		if (LocalIdentifierTable)
		{
			SerializeConcertIdentifierSource(EConcertIdentifierSource::HardcodedIndex_FixedSizeInt32);
			int32 ENameInt = (int32)*Ename;
			Ar << ENameInt; // Note: Don't serialize as a packed int so that the data size remains consistent in case this data gets rewritten (see FConcertIdentifierRewriter)
			bSerializeNumberAsPacked = false;
		}
		else
		{
			SerializeConcertIdentifierSource(EConcertIdentifierSource::HardcodedIndex_PackedInt);
			SerializeIndexValue((int32)*Ename);
		}
	}
	else if (LocalIdentifierTable == FConcertLocalIdentifierTable::ForceFNameTableIndex)
	{
		SerializeConcertIdentifierSource(EConcertIdentifierSource::FNameTableIndex_FixedSizeUInt32);
		uint32 FNameTableIndex = Name.GetDisplayIndex().ToUnstableInt();
		Ar << FNameTableIndex; // Note: Don't serialize as a packed int so that the data size remains consistent in case this data gets rewritten (see FConcertIdentifierRewriter)
		bSerializeNumberAsPacked = false;
	}
	else if (LocalIdentifierTable)
	{
		SerializeConcertIdentifierSource(EConcertIdentifierSource::LocalIdentifierTableIndex_FixedSizeInt32);
		int32 IdentifierTableIndex = LocalIdentifierTable->MapName(Name);
		Ar << IdentifierTableIndex; // Note: Don't serialize as a packed int so that the data size remains consistent in case this data gets rewritten (see FConcertIdentifierRewriter)
		bSerializeNumberAsPacked = false;
	}
	else
	{
		SerializeConcertIdentifierSource(EConcertIdentifierSource::PlainString);
		FString PlainString = Name.GetPlainNameString();
		Ar << PlainString;
	}
	
	int32 NameNumber = Name.GetNumber();
	if (bSerializeNumberAsPacked)
	{
		SerializeIndexValue(NameNumber);
	}
	else
	{
		Ar << NameNumber;
	}
}

void ReadName(FArchive& Ar, FName& Name, const FConcertLocalIdentifierTable* LocalIdentifierTable)
{
	checkf(LocalIdentifierTable != FConcertLocalIdentifierTable::ForceFNameTableIndex, TEXT("FConcertLocalIdentifierTable::ForceFNameTableIndex can only be used when writing!"));

	auto SerializeIndexValue = [&Ar]() -> int32
	{
		uint32 UnsignedIndex = 0;
		Ar.SerializeIntPacked(UnsignedIndex);
		return (int32)UnsignedIndex;
	};

	EConcertIdentifierSource Source;
	Ar.Serialize(&Source, sizeof(EConcertIdentifierSource));

	bool bSerializeNumberAsPacked = true;
	switch (Source)
	{
	case EConcertIdentifierSource::PlainString:
		{
			FString PlainString;
			Ar << PlainString;
			Name = FName(*PlainString, NAME_NO_NUMBER_INTERNAL, /*bSplitName*/false);
		}
		break;

	case EConcertIdentifierSource::HardcodedIndex_PackedInt:
		{
			const int32 HardcodedIndex = SerializeIndexValue();
			Name = EName(HardcodedIndex);
		}
		break;

	case EConcertIdentifierSource::HardcodedIndex_FixedSizeInt32:
		{
			int32 HardcodedIndex = INDEX_NONE;
			Ar << HardcodedIndex;
			Name = EName(HardcodedIndex);
			bSerializeNumberAsPacked = false;
		}
		break;

	case EConcertIdentifierSource::LocalIdentifierTableIndex_PackedInt:
		{
			const int32 IdentifierTableIndex = SerializeIndexValue();
			if (!LocalIdentifierTable || !LocalIdentifierTable->UnmapName(IdentifierTableIndex, Name))
			{
				Ar.SetError();
				return;
			}
		}
		break;

	case EConcertIdentifierSource::LocalIdentifierTableIndex_FixedSizeInt32:
		{
			int32 IdentifierTableIndex = INDEX_NONE;
			Ar << IdentifierTableIndex;
			if (!LocalIdentifierTable || !LocalIdentifierTable->UnmapName(IdentifierTableIndex, Name))
			{
				Ar.SetError();
				return;
			}
			bSerializeNumberAsPacked = false;
		}
		break;

	case EConcertIdentifierSource::FNameTableIndex_FixedSizeUInt32:
		{
			uint32 FNameTableIndex = 0;
			Ar << FNameTableIndex;
			Name = FName::CreateFromDisplayId(FNameEntryId::FromUnstableInt(FNameTableIndex), 0);
			bSerializeNumberAsPacked = false;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EConcertIdentifierSource!"));
		break;
	}

	int32 NameNumber = 0;
	if (bSerializeNumberAsPacked)
	{
		NameNumber = SerializeIndexValue();
	}
	else
	{
		Ar << NameNumber;
	}
	Name.SetNumber(NameNumber);
}

} // namespace UE::Concert::Private::ConcertIdentifierArchiveUtil

FConcertIdentifierWriter::FConcertIdentifierWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, TArray<uint8>& InBytes, bool bIsPersistent)
	: FMemoryWriter(InBytes, bIsPersistent)
	, LocalIdentifierTable(InLocalIdentifierTable)
{
}

FArchive& FConcertIdentifierWriter::operator<<(FName& Name)
{
	UE::Concert::Private::ConcertIdentifierArchiveUtil::WriteName(*this, Name, LocalIdentifierTable);
	return *this;
}

FArchive& FConcertIdentifierWriter::operator<<(FSoftObjectPath& AssetPtr)
{
	AssetPtr.SerializePath(*this);
	return *this;
}

FString FConcertIdentifierWriter::GetArchiveName() const
{
	return TEXT("FConcertIdentifierWriter");
}


FConcertIdentifierReader::FConcertIdentifierReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, const TArray<uint8>& InBytes, bool bIsPersistent)
	: FMemoryReader(InBytes, bIsPersistent)
	, LocalIdentifierTable(InLocalIdentifierTable)
{
}

FArchive& FConcertIdentifierReader::operator<<(FName& Name)
{
	if (GetError())
	{
		return *this;
	}

	UE::Concert::Private::ConcertIdentifierArchiveUtil::ReadName(*this, Name, LocalIdentifierTable);
	return *this;
}

FArchive& FConcertIdentifierReader::operator<<(FSoftObjectPath& AssetPtr)
{
	AssetPtr.SerializePath(*this);
	return *this;
}

FString FConcertIdentifierReader::GetArchiveName() const
{
	return TEXT("FConcertIdentifierReader");
}


FConcertIdentifierRewriter::FConcertIdentifierRewriter(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertLocalIdentifierTable* InRewriteIdentifierTable, TArray<uint8>& InBytes, bool bIsPersistent)
	: LocalIdentifierTable(InLocalIdentifierTable)
	, RewriteIdentifierTable(InRewriteIdentifierTable)
	, Bytes(InBytes)
{
	checkf(LocalIdentifierTable && RewriteIdentifierTable, TEXT("FConcertIdentifierRewriter can only rewrite data via an identifier table, as the rewritten data must be the same size!"));
	SetIsLoading(true);
	SetIsPersistent(bIsPersistent);
}

FArchive& FConcertIdentifierRewriter::operator<<(FName& Name)
{
	if (GetError())
	{
		return *this;
	}

	const int64 OffsetBeforeNameRead = Offset;
	UE::Concert::Private::ConcertIdentifierArchiveUtil::ReadName(*this, Name, LocalIdentifierTable);
	const int64 OffsetAfterNameRead = Offset;

	if (!LocalIdentifierTable->HasMappings())
	{
		// No mappings means nothing to rewrite
		return *this;
	}

	// Rewrite the data via the scratch buffer
	RewriteData(OffsetBeforeNameRead, OffsetAfterNameRead - OffsetBeforeNameRead, Name);
	return *this;
}

FArchive& FConcertIdentifierRewriter::operator<<(FSoftObjectPath& AssetPtr)
{
	AssetPtr.SerializePath(*this);
	return *this;
}

int64 FConcertIdentifierRewriter::TotalSize()
{
	return (int64)Bytes.Num();
}

void FConcertIdentifierRewriter::Serialize(void* Data, int64 Num)
{
	if (Num && !IsError())
	{
		// Only serialize if we have the requested amount of data
		if (Offset + Num <= TotalSize())
		{
			FMemory::Memcpy(Data, Bytes.GetData() + Offset, Num);
			Offset += Num;
		}
		else
		{
			SetError();
		}
	}
}

FString FConcertIdentifierRewriter::GetArchiveName() const
{
	return TEXT("FConcertIdentifierRewriter");
}
