// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scratchpad/ConcertScratchpad.h"
#include "Misc/ScopeLock.h"

bool FConcertScratchpad::HasValue(const FName& InId) const
{
	FScopeLock ScratchpadValuesLock(&ScratchpadValuesCS);
	return ScratchpadValues.Contains(InId);
}

void FConcertScratchpad::InternalSetValue(const FName& InId, IScratchpadValuePtr&& InValue)
{
	FScopeLock ScratchpadValuesLock(&ScratchpadValuesCS);
	ScratchpadValues.Emplace(InId, MoveTemp(InValue));
}

FConcertScratchpad::IScratchpadValuePtr FConcertScratchpad::InternalGetValue(const FName& InId) const
{
	FScopeLock ScratchpadValuesLock(&ScratchpadValuesCS);
	return ScratchpadValues.FindRef(InId);
}
