// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ConcertProtocolTrace.h"

namespace UE::ConcertTrace
{
	bool IsTracing(EProtocolSuite Id)
	{
		// At the moment it does not matter which protocol is being trace - we just always assume yes.
		// In the future this might change so keep the unused variable ID for now.
		return Trace::IsTracing() && Trace::IsChannel(TEXT("ConcertChannel"));
	}
}