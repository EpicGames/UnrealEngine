// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ConcertTrace.h"

UE_TRACE_CHANNEL_DEFINE(ConcertChannel, "Traces Concert multi-user collaboration events including protocol operations, network transport, \
transaction synchronization, connection initialization, and collaborative editing operations across connected users.");