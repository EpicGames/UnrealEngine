// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Private/ConcertProtocolTraceInternal.h"

#include "ConcertLogGlobal.h"
#include "Misc/DateTime.h"
#include "Trace/ConcertTrace.h"
#include "Trace/Private/InitEventTracking.h"

#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Trace.inl"

#if UE_CONCERT_TRACE_ENABLED

UE_TRACE_EVENT_BEGIN(ConcertLogger, Init, NoSync)
	/** EConcertTraceVersion */
	UE_TRACE_EVENT_FIELD(uint8, Version)
	UE_TRACE_EVENT_FIELD(uint32, EndpointId_A)
	UE_TRACE_EVENT_FIELD(uint32, EndpointId_B)
	UE_TRACE_EVENT_FIELD(uint32, EndpointId_C)
	UE_TRACE_EVENT_FIELD(uint32, EndpointId_D)
	/** Time the trace was started. Used to correlate times relative to other machines. */
	UE_TRACE_EVENT_FIELD(int64, DateTimeTicks)
	/** Cycle at which this init was generated. Used to correlate times relative to other machines. */
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ClientDisplayName)
	UE_TRACE_EVENT_FIELD(bool, IsServer)
	UE_TRACE_EVENT_FIELD(bool, HasEndpointId)
	UE_TRACE_EVENT_FIELD(bool, HasDisplayName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(ConcertLogger, ObjectTransmissionStart, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, DestEndpointId_A)
	UE_TRACE_EVENT_FIELD(uint32, DestEndpointId_B)
	UE_TRACE_EVENT_FIELD(uint32, DestEndpointId_C)
	UE_TRACE_EVENT_FIELD(uint32, DestEndpointId_D)
	UE_TRACE_EVENT_FIELD(uint8, Protocol)
	UE_TRACE_EVENT_FIELD(uint32, SequenceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(ConcertLogger, ObjectTransmissionReceive, NoSync)
	UE_TRACE_EVENT_FIELD(uint8, Protocol)
	UE_TRACE_EVENT_FIELD(uint32, SequenceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(ConcertLogger, ObjectSink, NoSync)
	UE_TRACE_EVENT_FIELD(uint8, Protocol)
	UE_TRACE_EVENT_FIELD(uint32, SequenceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, SinkName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

namespace UE::ConcertTrace
{
	void Init(const TOptional<FGuid>& LocalEndpointId, const FString& ClientDisplayName, bool bIsServer)
	{
		const bool bIsConnected = FTraceAuxiliary::IsConnected();
		const bool bHasSentInitEventBefore = HasSentInitEventToCurrentSession();
		if (!bIsConnected
			|| bHasSentInitEventBefore)
		{
			return;
		}
		
		const bool bIsChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(ConcertChannel);
		if (!bIsChannelEnabled)
		{
			// Warn so developer can debug it
			UE_LOGF(LogConcert, Warning, "Skipping Concert init event because the Concert channel is not enabled.");
			return;
		}
		
		OnSendInitEvent();
		
		UE_TRACE_LOG(ConcertLogger, Init, ConcertChannel)
		<< Init.Version(static_cast<uint8>(EConcertTraceVersion::Latest))
		<< Init.EndpointId_A(LocalEndpointId.IsSet() ? LocalEndpointId->A : 0) 
		<< Init.EndpointId_B(LocalEndpointId.IsSet() ? LocalEndpointId->B : 0) 
		<< Init.EndpointId_C(LocalEndpointId.IsSet() ? LocalEndpointId->C : 0) 
		<< Init.EndpointId_D(LocalEndpointId.IsSet() ? LocalEndpointId->D : 0)
		<< Init.DateTimeTicks(FDateTime::UtcNow().GetTicks())
		<< Init.Cycle(FPlatformTime::Cycles64())
		<< Init.ClientDisplayName(*ClientDisplayName)
		<< Init.IsServer(bIsServer) 
		<< Init.HasEndpointId(LocalEndpointId.IsSet())
		<< Init.HasDisplayName(!ClientDisplayName.IsEmpty());
	}
	
	void LogTransmissionStart(uint8 Protocol, uint32 SequenceId, const FSoftObjectPath& ObjectPath, const FGuid& TargetEndpointId)
	{
		if (!ShouldTraceConcertProtocols())
		{
			return;
		}
		
		UE_TRACE_LOG(ConcertLogger, ObjectTransmissionStart, ConcertChannel)
		<< ObjectTransmissionStart.DestEndpointId_A(TargetEndpointId.A) 
		<< ObjectTransmissionStart.DestEndpointId_B(TargetEndpointId.B) 
		<< ObjectTransmissionStart.DestEndpointId_C(TargetEndpointId.C) 
		<< ObjectTransmissionStart.DestEndpointId_D(TargetEndpointId.D) 
		<< ObjectTransmissionStart.Protocol(Protocol) 
		<< ObjectTransmissionStart.SequenceId(SequenceId) 
		<< ObjectTransmissionStart.ObjectPath(*ObjectPath.ToString())
		<< ObjectTransmissionStart.Cycle(FPlatformTime::Cycles64());
	}
	
	void LogTransmissionReceive(uint8 Protocol, uint32 SequenceId, const FSoftObjectPath& ObjectPath)
	{
		if (!ShouldTraceConcertProtocols())
		{
			return;
		}
		
		UE_TRACE_LOG(ConcertLogger, ObjectTransmissionReceive, ConcertChannel) 
		<< ObjectTransmissionReceive.Protocol(Protocol) 
		<< ObjectTransmissionReceive.SequenceId(SequenceId) 
		<< ObjectTransmissionReceive.ObjectPath(*ObjectPath.ToString())
		<< ObjectTransmissionReceive.Cycle(FPlatformTime::Cycles64());
	}
	
	void LogSink(uint8 Protocol, uint32 SequenceId, const ANSICHAR* SinkName, const FSoftObjectPath& ObjectPath)
	{
		if (!ShouldTraceConcertProtocols())
		{
			return;
		}
		
		UE_TRACE_LOG(ConcertLogger, ObjectSink, ConcertChannel) 
		<< ObjectSink.Protocol(Protocol) 
		<< ObjectSink.SequenceId(SequenceId) 
		<< ObjectSink.SinkName(SinkName) 
		<< ObjectSink.ObjectPath(*ObjectPath.ToString())
		<< ObjectSink.Cycle(FPlatformTime::Cycles64());
	}
}

#endif