// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Private/ConcertScopedObjectTrace.h"

#include "Trace/ConcertTrace.h"

#if UE_CONCERT_TRACE_ENABLED

#include "Trace/Private/InitEventTracking.h"

#include "HAL/PlatformTime.h"
#include "Trace/Trace.inl"

UE_TRACE_EVENT_BEGIN(ConcertLogger, ObjectTraceBegin, NoSync)
	UE_TRACE_EVENT_FIELD(uint8, Protocol)
	UE_TRACE_EVENT_FIELD(uint32, SequenceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, EventName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(ConcertLogger, ObjectTraceEnd, NoSync)
	UE_TRACE_EVENT_FIELD(uint8, Protocol)
	UE_TRACE_EVENT_FIELD(uint32, SequenceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, EventName)
UE_TRACE_EVENT_END()

namespace UE::ConcertTrace
{
	FConcertScopedObjectTrace::FConcertScopedObjectTrace(
		uint8 InProtocol,
		uint32 InSequenceId,
		const ANSICHAR* InEventName,
		FSoftObjectPath InObjectPath
		)
		: bShouldTrace(ShouldTraceConcertProtocols())
		, Protocol(InProtocol)
		, SequenceId(InSequenceId)
		, EventName(InEventName)
		, ObjectPath(MoveTemp(InObjectPath))
	{
		if (!bShouldTrace)
		{
			return;
		}
		
		// Send one event in constructor with begin time and one in destructor with end time instead of one event in destructor that has begin and end time.
		// Reason: Needed to for TMonotonicTimeline::AppendBeginEvent API used internally by FProtocolMultiEndpointProvider for nested CPU events. 
		UE_TRACE_LOG(ConcertLogger, ObjectTraceBegin, ConcertChannel)
		<< ObjectTraceBegin.Protocol(Protocol) 
		<< ObjectTraceBegin.SequenceId(SequenceId) 
		<< ObjectTraceBegin.ObjectPath(*ObjectPath.ToString())
		<< ObjectTraceBegin.Cycle(FPlatformTime::Cycles64())
		<< ObjectTraceBegin.EventName(EventName);
	}

	FConcertScopedObjectTrace::~FConcertScopedObjectTrace()
	{
		if (!bShouldTrace)
		{
			return;
		}
		
		UE_TRACE_LOG(ConcertLogger, ObjectTraceEnd, ConcertChannel)
		<< ObjectTraceEnd.Protocol(Protocol) 
		<< ObjectTraceEnd.SequenceId(SequenceId) 
		<< ObjectTraceEnd.ObjectPath(*ObjectPath.ToString())
		<< ObjectTraceEnd.Cycle(FPlatformTime::Cycles64())
		<< ObjectTraceEnd.EventName(EventName);
	}
}
#endif