// Copyright Epic Games, Inc. All Rights Reserved.

#include "InitEventTracking.h"

#include "Trace/ConcertTraceConfig.h"

#if UE_CONCERT_TRACE_ENABLED

#include "ConcertLogGlobal.h"
#include "Trace/ConcertTrace.h"

#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Trace.h"

namespace UE::ConcertTrace
{
	/** Trace destinations that the local application instance has already sent the init event to. The init event should only appear in every .utrace file once. */
	TSet<FString> TraceDestinationsWithInitEvent;

	bool HasSentInitEventToCurrentSession()
	{
		return TraceDestinationsWithInitEvent.Contains(FTraceAuxiliary::GetTraceDestinationString());
	}
	
	bool ShouldTraceConcertProtocols()
	{
		return UE_TRACE_CHANNELEXPR_IS_ENABLED(ConcertChannel)
			&& HasSentInitEventToCurrentSession();
	}

	void OnSendInitEvent()
	{
		UE_LOGF(LogConcert, Log, "Sending trace init event");
		TraceDestinationsWithInitEvent.Add(FTraceAuxiliary::GetTraceDestinationString());
	}
}

#endif