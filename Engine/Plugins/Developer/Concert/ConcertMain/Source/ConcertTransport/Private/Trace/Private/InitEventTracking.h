// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/ConcertTraceConfig.h"

#if UE_CONCERT_TRACE_ENABLED

namespace UE::ConcertTrace
{
	/** Checks whether an init event has been sent for the current trace session already. */
	bool HasSentInitEventToCurrentSession();

	/** @return Whether protocol tracing should generate data. */
	bool ShouldTraceConcertProtocols();

	/** Tracks that for the current tracing session, and init event was been sent. */
	void OnSendInitEvent();
}

#endif
