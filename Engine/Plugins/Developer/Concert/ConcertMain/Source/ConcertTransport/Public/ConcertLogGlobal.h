// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
// This include is for backwards compatibility. SCOPED_CONCERT_TRACE and ConcertChannel used to be defined here. TODO DP: Deprecate this correctly
#include "Trace/ConcertTrace.h" 

// The default debug verbosity level can be set as a GlobalDefinitions in an executable Target.cs build file to adapt to this executable context.
#ifndef UE_LOG_CONCERT_DEBUG_VERBOSITY_LEVEL
#define UE_LOG_CONCERT_DEBUG_VERBOSITY_LEVEL Error
#endif

CONCERTTRANSPORT_API DECLARE_LOG_CATEGORY_EXTERN(LogConcert, Log, All);
CONCERTTRANSPORT_API DECLARE_LOG_CATEGORY_EXTERN(LogConcertDebug, UE_LOG_CONCERT_DEBUG_VERBOSITY_LEVEL, All);
