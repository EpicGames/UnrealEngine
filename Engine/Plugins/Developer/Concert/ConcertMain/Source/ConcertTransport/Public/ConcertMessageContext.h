// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTransportMessages.h"

/** Context struct for Concert messages */
struct FConcertMessageContext
{
	/** Construct an empty context */
	FConcertMessageContext()
		: SenderConcertEndpointId()
		, UtcNow(0)
		, Message(nullptr)
		, MessageType(nullptr)
	{
	}

	/** Construct a context with a reference to the given message data */
	FConcertMessageContext(const FGuid& InSenderConcertEndpointId, const FDateTime& InUtcNow, const FConcertMessageData* InMessage, const UScriptStruct* InMessageType, const TMap<FName, FString>& InAnnotations)
		: SenderConcertEndpointId(InSenderConcertEndpointId)
		, UtcNow(InUtcNow)
		, Message(InMessage)
		, MessageType(InMessageType)
		, Annotations(InAnnotations)
	{
	}

	/** Utility to get the message data cast to the correct type */
	template <typename T>
	const T* GetMessage() const
	{
		check(MessageType->IsChildOf(T::StaticStruct()));
		return static_cast<const T*>(Message);
	}

	FGuid SenderConcertEndpointId;
	FDateTime UtcNow;
	const FConcertMessageData* Message;
	const UScriptStruct* MessageType;
	TMap<FName, FString> Annotations;
};

struct FConcertMessageCapturedContext
{
public:
	explicit FConcertMessageCapturedContext(const FConcertMessageContext& InContext);
	~FConcertMessageCapturedContext();

	FConcertMessageCapturedContext(const FConcertMessageCapturedContext& Other) = delete;
	const FConcertMessageCapturedContext& operator=(const FConcertMessageCapturedContext& Other) = delete;

	FConcertMessageCapturedContext(FConcertMessageCapturedContext&& Other);
	const FConcertMessageCapturedContext& operator=(FConcertMessageCapturedContext&& Other);

	FConcertMessageContext CapturedContext;

private:
	void MallocCapturedContextMessage(const FConcertMessageData* InMessage, const UScriptStruct* InMessageType);
	void FreeCapturedContextMessage();
};
