// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Misc/Guid.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "ConcertTransportMessages.generated.h"

/** Message Flags when sent */
UENUM()
enum class EConcertMessageFlags : uint8
{
	/** No special flags */
	None = 0,
	/** Guarantee that this message is received by the client(s) and processed in the order they were sent */
	ReliableOrdered = 1<<0,
	/** Message sent with this flag should be uniquely identifiable across clients. */
	UniqueId = 1<<1, 
};
ENUM_CLASS_FLAGS(EConcertMessageFlags);

/** Response codes to a sent request */
UENUM()
enum class EConcertResponseCode : uint8
{
	/** The response code is still pending. */
	Pending,
	/** The request data was valid. A response was generated. */
	Success,
	/** The request data was valid, but the request failed. A response was generated. */
	Failed,
	/** The request data was invalid. No response was generated. */
	InvalidRequest,
	/** The request type was unknown on the target instance. No response was generated. */
	UnknownRequest,
	/** The request failed to reach the target instance. No response was generated. */
	TimedOut,
};

/** Handshake codes used during reliable channel negotiation */
UENUM()
enum class EConcertReliableHandshakeState : uint8
{
	/** Handshake initialization state */
	None,
	/** Handshake is being negotiated */
	Negotiate,
	/** Handshake was successfully negotiated */
	Success,
};

/** Versioning for concert message protocol */
UENUM()
enum class EConcertMessageVersion : uint32
{
	BeforeVersioning = 0,
	Initial, 
	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

/** Base class for all message data sent through concert */
USTRUCT()
struct FConcertMessageData
{
	GENERATED_BODY()

	FConcertMessageData()
		: MessageOrderIndex(0)
		, ChannelId(UnreliableChannelId)
	{
	}
	
	virtual ~FConcertMessageData() = default;

	bool IsReliable() const
	{
		return ChannelId != UnreliableChannelId;
	}

	EConcertMessageFlags GetMessageFlags() const
	{
		return IsReliable() ? EConcertMessageFlags::ReliableOrdered : EConcertMessageFlags::None;
	}

	virtual bool IsSafeToHandle() const
	{
		return true;
	}

	/** ID of the Endpoint this was sent from */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	FGuid ConcertEndpointId;
	
	/** ID of the message */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	FGuid MessageId;

	/** Order index of the message (for ordering reliable messages, used when ChannelId != UnreliableChannelId) */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	uint16 MessageOrderIndex;

	/** ID of the channel this message was sent from */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	uint16 ChannelId;

	/** Constant to represent a unreliable channel ID */
	static const uint16 UnreliableChannelId = 0;
};

/** Base struct for all concert event messages */
USTRUCT()
struct FConcertEventData : public FConcertMessageData
{
	GENERATED_BODY()
};

/** Base struct for all concert request messages */
USTRUCT()
struct FConcertRequestData : public FConcertMessageData
{
	GENERATED_BODY()
};

/** Base struct for all concert request response messages */
USTRUCT()
struct FConcertResponseData : public FConcertMessageData
{
	GENERATED_BODY()

	FConcertResponseData()
		: ResponseCode(EConcertResponseCode::Pending)
	{
	}

	FConcertResponseData(EConcertResponseCode InResponseCode)
		: ResponseCode(InResponseCode)
	{
	}

	/** Utility to get resolved response data as a future */
	template <typename ResponseType>
	static FORCEINLINE TFuture<ResponseType> AsFuture(ResponseType&& ResponseData)
	{
		return MakeFulfilledPromise<ResponseType>(Forward<ResponseType>(ResponseData)).GetFuture();
	}

	/** ID of the request message we're responding to */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	FGuid RequestMessageId;

	/** Response code for the response */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	EConcertResponseCode ResponseCode;

	/** If the code isn't successful, a reason for it */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	FText Reason;
};

/** Special event message base struct that is also caught by the endpoint to discover remote endpoint before passing it to handlers */
USTRUCT()
struct FConcertEndpointDiscoveryEvent : public FConcertEventData
{
	GENERATED_BODY()

	/** Holds the concert messages protocol version, default initialize to `BeforeVersioning` to handle message sent from older protocol. */
	UPROPERTY()
	EConcertMessageVersion ConcertProtocolVersion = EConcertMessageVersion::BeforeVersioning;
};

/** Message send when an endpoint is closed on a remote peer */
USTRUCT()
struct FConcertEndpointClosedData : public FConcertMessageData
{
	GENERATED_BODY()
};

/** Handshake used to negotiate a reliable channel between endpoints (also uses the ReliableChannelId from the base message) */
USTRUCT()
struct FConcertReliableHandshakeData : public FConcertEndpointDiscoveryEvent
{
	GENERATED_BODY()

	/** State of the handshake */
	UPROPERTY()
	EConcertReliableHandshakeState HandshakeState = EConcertReliableHandshakeState::None;

	/** Channel ID we're going to send reliable messages of */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	uint16 ReliableChannelId = 0;

	/** The next message index that the remote endpoint is going to send */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	uint16 NextMessageIndex = 0;

	/** It's a timespan encoded in ticks. EndpointTimeoutTick represent the time it takes for the sending endpoint to consider another endpoint timed out */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Concert Message")
	int64 EndpointTimeoutTick = 0;
};

/** Acknowledgment messages to reliable events */
USTRUCT()
struct FConcertAckData : public FConcertMessageData
{
	GENERATED_BODY()

	/** Time when this acknowledgment was sent (UTC) */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	int64 AckSendTimeTicks  = 0;

	/** ID of the source message we're acknowledging */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Concert Message")
	FGuid SourceMessageId;
};

/** Keep alive message */
USTRUCT()
struct FConcertKeepAlive : public FConcertMessageData
{
	GENERATED_BODY()
};

/** Force the endpoint to resend pending packages.*/
USTRUCT()
struct FConcertSendResendPending
{
	GENERATED_BODY()
};
