// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "ConcertTransportSettings.generated.h"


USTRUCT()
struct FConcertEndpointSettings
{
	GENERATED_BODY()

	FConcertEndpointSettings()
		: bEnableLogging(false)
		, PurgeProcessedMessageDelaySeconds(30)
		, RemoteEndpointTimeoutSeconds(60)
	{}

	/** Enable detailed message logging for Concert endpoints */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="Endpoint Settings")
	bool bEnableLogging;

	/** The timespan at which retained processed messages are purged on Concert endpoints */
	UPROPERTY(config, EditAnywhere, DisplayName="Purge Processed Message Delay", AdvancedDisplay, Category="Endpoint Settings", meta=(ForceUnits=s))
	int32 PurgeProcessedMessageDelaySeconds;

	/** The timespan at which remote endpoints that haven't sent a message are considered stale */
	UPROPERTY(config, EditAnywhere, DisplayName="Remote Endpoint Timeout", AdvancedDisplay, Category="Endpoint Settings", meta=(ForceUnits=s, ClampMin=4, UIMin=4))
	int32 RemoteEndpointTimeoutSeconds;
};

UCLASS(config=Engine)
class UConcertEndpointConfig : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category="Endpoint Settings", meta=(ShowOnlyInnerProperties))
	FConcertEndpointSettings EndpointSettings;
};
