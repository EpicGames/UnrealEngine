// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "ConcertTransportMessages.h"
#include "IConcertMessageHandler.h"
#include "IConcertMessages.h"

struct FConcertMessageContext;

/**
 * Context information for a Concert endpoint
 */
struct FConcertEndpointContext
{
	/** Convert this endpoint context into a string in the form "{FriendlyName} ({Id})" */
	FString ToString() const
	{
		return FString::Printf(TEXT("%s (%s)"), *EndpointFriendlyName, *EndpointId.ToString());
	}

	/** ID of this endpoint */
	FGuid EndpointId;

	/** Friendly name of this endpoint (not guaranteed to be unique) */
	FString EndpointFriendlyName;
};

/** 
 * Remote endpoint connection statuses when broadcasting changes
 */
enum class EConcertRemoteEndpointConnection : uint8
{
	/** The remote endpoint was discovered. */
	Discovered,
	/** The remote endpoint timed-out. */
	TimedOut,
	/** The remote endpoint was closed by the remote peer. */
	ClosedRemotely,
};

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConcertMessageAcknowledgementReceived,
	const FConcertEndpointContext& /*RemoteEndpoint*/,
	const TSharedRef<IConcertMessage>& /*AckedMessage*/,
	const FConcertMessageContext& /*AckMessageContext*/
	);

/** 
 * Interface representing a remote endpoint
 * that you can send to reliably or not from a local Concert endpoint
 */
class IConcertRemoteEndpoint
{
public:
	/** Virtual destructor */
	virtual ~IConcertRemoteEndpoint() = default;
	
	/** Get the context for this remote endpoint */
	virtual const FConcertEndpointContext& GetEndpointContext() const = 0;
	
	/** Callback when a message has been acknowledged by this remote endpoint */
	virtual FOnConcertMessageAcknowledgementReceived& OnConcertMessageAcknowledgementReceived() = 0;

protected:
	/** 
	 * Set the message order index, for friend class 
	 */
	static void SetMessageOrderIndex(const TSharedRef<IConcertMessage>& Message, uint16 OrderIndex)
	{
		Message->SetOrderIndex(OrderIndex);
	}

	/** 
	 * Set the message channel ID, for friend class 
	 */
	static void SetMessageChannelId(const TSharedRef<IConcertMessage>& Message, uint16 ChannelId)
	{
		Message->SetChannelId(ChannelId);
	}
};


DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertRemoteEndpointConnectionChanged, const FConcertEndpointContext&, EConcertRemoteEndpointConnection);
DECLARE_MULTICAST_DELEGATE_FourParams(FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint,
	const FConcertEndpointContext& /*LocalEndpoint*/,
	const FConcertEndpointContext& /*RemoteEndpoint*/,
	const TSharedRef<IConcertMessage>& /*AckedMessage*/,
	const FConcertMessageContext& /*MessageContext*/
);

/**
 * Interface representing a local endpoint you can 
 * send from either reliably or not for Concert
 */
class IConcertLocalEndpoint
{
public:
	/** Virtual destructor */
	virtual ~IConcertLocalEndpoint() = default;

	/** Get the context for this endpoint. */
	virtual const FConcertEndpointContext& GetEndpointContext() const = 0;

	/** Gets all remote endpoints connected to this local endpoint. */
	virtual TArray<FConcertEndpointContext> GetRemoteEndpoints() const = 0;
	
	/** Gets the remote address of a client or the server. If ConcertEndpointId is not valid, the result's IsValid return false. */
	virtual FMessageAddress GetRemoteAddress(const FGuid& ConcertEndpointId) const = 0;

	/** Callback when a remote endpoint connection changes. */
	virtual FOnConcertRemoteEndpointConnectionChanged& OnRemoteEndpointConnectionChanged() = 0;

	/** Callback when a message has been acknowledged by a remote endpoint */
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() = 0;

	/**
	 * Send a request to a remote endpoint
	 * @param Request : the request to send, needs to be a UStruct that derives from FConcertRequestData
	 * @param Endpoint : the remote endpoint Id to send to
	 * @return A future of the response for the request
	 */
	template<typename RequestType, typename ResponseType>
	TFuture<ResponseType> SendRequest(const RequestType& Request, const FGuid& Endpoint) 	// TODO: Allow moving
	{
		static_assert(TIsDerivedFrom<RequestType, FConcertRequestData>::IsDerived, "Sent RequestType need to be a UStruct deriving of FConcertRequestData.");
		typedef TConcertRequest<RequestType, ResponseType> ConcertRequestType;
		TSharedRef<ConcertRequestType> RequestRef = MakeShared<ConcertRequestType>(Request);

		InternalQueueRequest(RequestRef, Endpoint);
		return RequestRef->GetFuture();
	}

	/**
	 * Send an event to a remote endpoint
	 * @param Event : the event to send, needs to be a UStruct that derives from FConcertEventData
	 * @param Endpoint : the remote endpoint Id to send to
	 * @param Flags : flags for the event (i.e.: is the event reliable)
	 * @param Annotations : Annotations to attach to the message.
	 */
	template<typename EventType>
	void SendEvent(const EventType& Event, const FGuid& Endpoint, EConcertMessageFlags Flags = EConcertMessageFlags::None, TMap<FName, FString> Annotations = TMap<FName, FString>())
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "Sent EventType need to be a UStruct deriving of FConcertEventData.");
		typedef TConcertEvent<EventType> ConcertEventType;
		TSharedRef<ConcertEventType> EventRef = MakeShared<ConcertEventType>(Event);

		EventRef.Get().SetAnnotations(MoveTemp(Annotations));
		InternalQueueEvent(EventRef, Endpoint, Flags);
	}

	/**
	 * Publish an event, other remote endpoint need to subscribe to the event to receive it
	 * @param Event : the event to publish, needs to be a UStruct that derives from FConcertEventData
	 */
	template<typename EventType>
	void PublishEvent(const EventType& Event) // TODO: Allow moving
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "Sent EventType need to be a UStruct deriving of FConcertEventData.");
		typedef TConcertEvent<EventType> ConcertEventType;
		TSharedRef<ConcertEventType> EventRef = MakeShared<ConcertEventType>(Event);

		InternalPublishEvent(EventRef);
	}

	/**
	 * Register an handler for request received by this endpoint
	 * @param Func	  : Function to handle the request of type RequestType
	 */
	template<typename RequestType, typename ResponseType>
	void RegisterRequestHandler(typename TConcertFunctionRequestHandler<ResponseType>::FFuncType Func)
	{
		static_assert(TIsDerivedFrom<RequestType, FConcertRequestData>::IsDerived, "RequestType need to be a UStruct deriving of FConcertRequestData.");
		InternalAddRequestHandler(RequestType::StaticStruct()->GetStructPathName(), MakeShared<TConcertFunctionRequestHandler<ResponseType>>(MoveTemp(Func)));
	}

	/**
	 * Register an handler for request received by this endpoint
	 * @param Handler : Pointer for the handler
	 * @param Func	  : Member function of the Handler to handle the request of type RequestType
	 */
	template<typename RequestType, typename ResponseType, typename HandlerType>
	void RegisterRequestHandler(HandlerType* Handler, typename TConcertRawRequestHandler<ResponseType, HandlerType>::FFuncType Func)
	{
		static_assert(TIsDerivedFrom<RequestType, FConcertRequestData>::IsDerived, "RequestType need to be a UStruct deriving of FConcertRequestData.");
		InternalAddRequestHandler(RequestType::StaticStruct()->GetStructPathName(), MakeShared<TConcertRawRequestHandler<ResponseType, HandlerType>>(Handler, Func));
	}

	/**
	 * Unregister the handler for request received by this endpoint of type RequestType
	 */
	template<typename RequestType>
	void UnregisterRequestHandler()
	{
		static_assert(TIsDerivedFrom<RequestType, FConcertRequestData>::IsDerived, "RequestType need to be a UStruct deriving of FConcertRequestData.");
		InternalRemoveRequestHandler(RequestType::StaticStruct()->GetStructPathName());
	}

	/**
	 * Register an handler for event received by this endpoint
	 * @param Func	  : Function to handle the event of type EventType
	 */
	template<typename EventType>
	void RegisterEventHandler(typename TConcertFunctionEventHandler::FFuncType Func)
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "EventType need to be a UStruct deriving of FConcertEventData.");
		InternalAddEventHandler(EventType::StaticStruct()->GetStructPathName(), MakeShared<TConcertFunctionEventHandler>(MoveTemp(Func)));
	}

	/**
	 * Register an handler for event received by this endpoint
	 * @param Handler : Pointer for the handler
	 * @param Func	  : Member function of the Handler to handle the event of type EventType
	 */
	template<typename EventType, typename HandlerType>
	void RegisterEventHandler(HandlerType* Handler, typename TConcertRawEventHandler<HandlerType>::FFuncType Func)
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "EventType need to be a UStruct deriving of FConcertEventData.");
		InternalAddEventHandler(EventType::StaticStruct()->GetStructPathName(), MakeShared<TConcertRawEventHandler<HandlerType>>(Handler, Func));
	}

	/**
	 * Unregister the handler for event received by this endpoint of type EventType
	 */
	template<typename EventType>
	void UnregisterEventHandler()
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "EventType need to be a UStruct deriving of FConcertEventData.");
		InternalRemoveEventHandler(EventType::StaticStruct()->GetStructPathName());
	}

	/**
	 * Subscribe an handler for event received by this endpoint, this will handle published event
	 * @param Handler : Pointer for the handler
	 * @param Func	  : Member function of the Handler to handle the event of type EventType
	 */
	template<typename EventType, typename HandlerType>
	void SubscribeEventHandler(HandlerType* Handler, typename TConcertRawEventHandler<HandlerType>::FFuncType Func)
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "EventType need to be a UStruct deriving of FConcertEventData.");
		InternalAddEventHandler(EventType::StaticStruct()->GetStructPathName(), MakeShared<TConcertRawEventHandler<HandlerType>>(Handler, Func));
		InternalSubscribeToEvent(EventType::StaticStruct()->GetStructPathName());
	}

	// TODO TFunction EventHandler

	/**
	 * Unregister the handler for event received by this endpoint of type EventType
	 */
	template<typename EventType>
	void UnsubscribeEventHandler()
	{
		static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "EventType need to be a UStruct deriving of FConcertEventData.");

		// Names can be invalid if unregistering during shutdown
		const FTopLevelAssetPath EventName = EventType::StaticStruct()->GetStructPathName();
		if (EventName.IsValid())
		{
			InternalRemoveEventHandler(EventName);
			InternalUnsubscribeFromEvent(EventName);
		}
	}

protected:
	/** Set message ID and sender ID */
	void SetMessageSendingInfo(const TSharedRef<IConcertMessage>& Message)
	{
		Message->SetMessageId(FGuid::NewGuid());
		Message->SetSenderId(GetEndpointContext().EndpointId);
	}

	/** Set message ID and sender ID */
	void SetResponseSendingInfo(const TSharedRef<IConcertResponse>& Response, const FGuid& RequestMessageId)
	{
		SetMessageSendingInfo(Response);
		Response->SetRequestMessageId(RequestMessageId);
	}

	/**
	 * Add a Request Handler
	 */
	virtual void InternalAddRequestHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IConcertRequestHandler>& Handler) = 0;

	/**
	 * Remove an Request Handler
	 */
	virtual void InternalRemoveRequestHandler(const FTopLevelAssetPath& RequestMessageType) = 0;

	/**
	 * Add an Event Handler
	 */
	virtual void InternalAddEventHandler(const FTopLevelAssetPath& EventMessageType, const TSharedRef<IConcertEventHandler>& Handler) = 0;

	/**
	 * Remove an Event Handler
	 */
	virtual void InternalRemoveEventHandler(const FTopLevelAssetPath& EventMessageType) = 0;

	/**
	 * Subscribe to an Event
	 */
	virtual void InternalSubscribeToEvent(const FTopLevelAssetPath& EventMessageType) = 0;

	/**
	 * Unsubscribe from an Event
	 */
	virtual void InternalUnsubscribeFromEvent(const FTopLevelAssetPath& EventMessageType) = 0;

	/**
	 * Queue a request to be sent to a remote endpoint
	 * @param Request : the type erased request
	 * @param Endpoint : the remote endpoint Id to send to
	 * @return A future of the response for the request
	 */
	virtual void InternalQueueRequest(const TSharedRef<IConcertRequest>& Request, const FGuid& Endpoint) = 0;

	/**
	 * Queue a response to be sent back to a remote endpoint
	 * @param Response : the type erased response
	 * @param Endpoint : the remote endpoint Id to send to
	 * @return A future of the response for the request
	 */
	virtual void InternalQueueResponse(const TSharedRef<IConcertResponse>& Response, const FGuid& Endpoint) = 0;

	/**
	 * Queue an event to be sent to a remote endpoint
	 * @param Event : the type erased event
	 * @param Endpoint : the remote endpoint Id to send to
	 * Flags : flags for the event (i.e.: is the event reliable)
	 */
	virtual void InternalQueueEvent(const TSharedRef<IConcertEvent>& Event, const FGuid& Endpoint, EConcertMessageFlags Flags) = 0;

	/**
	 * Publish an event to any listening endpoints
	 * @param Event : the type erased event	 
	 */
	virtual void InternalPublishEvent(const TSharedRef<IConcertEvent>& Event) = 0;
};

typedef TSharedPtr<IConcertLocalEndpoint> IConcertLocalEndpointPtr; // TODO: need thread safe?
