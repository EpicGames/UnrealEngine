// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "IConcertMessages.h"

class IMessageContext;
class IConcertResponse;

/** 
 * Interface for Concert Endpoint request handler
 */
class IConcertRequestHandler
{
public:
	virtual ~IConcertRequestHandler() = default;

	/** 
	 * Handle the request from the context passed
	 * @param Context the concert message context, the message needs to be casted
	 * @return a shared ptr to a Concert Response
	 */
	virtual TFuture<TSharedPtr<IConcertResponse>> HandleRequest(const FConcertMessageContext& Context) const = 0;
};

/**
* Interface for Concert Endpoint event handler
*/
class IConcertEventHandler
{
public:
	virtual ~IConcertEventHandler() = default;

	/**
	 * Handle the request from the context passed
	 * @param Context the concert message context, the message needs to be casted
	 */
	virtual void HandleEvent(const FConcertMessageContext& Context) const = 0;
};

/**
 * Implementation of Concert Endpoint event handler using a TFunction
 */
class TConcertFunctionEventHandler : public IConcertEventHandler
{
public:
	/** Type definition for function pointers that are compatible with Event of this TConcertRawEventHandler. */
	typedef TFunction<void(const FConcertMessageContext&)> FFuncType;

	TConcertFunctionEventHandler(FFuncType InFunc)
		: Func(MoveTemp(InFunc))
	{}
	virtual ~TConcertFunctionEventHandler() = default;

	virtual void HandleEvent(const FConcertMessageContext& Context) const
	{
		// TODO: template on message type and add checks?
		Func(Context);
	}

private:
	FFuncType Func;
};

/**
 * Implementation of Concert Endpoint event handler using a raw member function pointer
 */
template<typename HandlerType>
class TConcertRawEventHandler : public IConcertEventHandler
{
public:
	/** Type definition for function pointers that are compatible with Event of this TConcertRawEventHandler. */
	typedef void (HandlerType::*FFuncType)(const FConcertMessageContext&);

	TConcertRawEventHandler(HandlerType* InHandler, FFuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(InHandler != nullptr);
	}
	virtual ~TConcertRawEventHandler() = default;

	virtual void HandleEvent(const FConcertMessageContext& Context) const
	{
		(Handler->*Func)(Context);
	}

private:
	HandlerType* Handler;
	FFuncType Func;
};

/**
 * Implementation of Concert Endpoint request handler using TFunction
 */
template<typename ResponseType>
class TConcertFunctionRequestHandler : public IConcertRequestHandler
{
public:
	/** Type definition for function pointers that are compatible with Request of this TConcertRawRequestHandler. */
	typedef TFunction<TFuture<ResponseType>(const FConcertMessageContext&)> FFuncType;

	TConcertFunctionRequestHandler(FFuncType InFunc)
		: Func(MoveTemp(InFunc))
	{}
	virtual ~TConcertFunctionRequestHandler() = default;

	virtual TFuture<TSharedPtr<IConcertResponse>> HandleRequest(const FConcertMessageContext& Context) const
	{
		return Func(Context)
			.Next([](ResponseType&& Response)
			{
				return TSharedPtr<IConcertResponse>(MakeShared<TConcertResponse<ResponseType>>(Forward<ResponseType>(Response)));
			});
	}

private:
	FFuncType Func;
};

/**
 * Implementation of Concert Endpoint request handler using a raw member function pointer
 */
template<typename ResponseType, typename HandlerType>
class TConcertRawRequestHandler : public IConcertRequestHandler
{
public:
	/** Type definition for function pointers that are compatible with Event of this TConcertRawRequestHandler. */
	typedef TFuture<ResponseType>(HandlerType::*FFuncType)(const FConcertMessageContext&);

	TConcertRawRequestHandler(HandlerType* InHandler, FFuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(InHandler != nullptr);
	}
	virtual ~TConcertRawRequestHandler() = default;

	virtual TFuture<TSharedPtr<IConcertResponse>> HandleRequest(const FConcertMessageContext& Context) const
	{
		return (Handler->*Func)(Context)
			.Next([](ResponseType&& Response)
			{
				return TSharedPtr<IConcertResponse>(MakeShared<TConcertResponse<ResponseType>>(Forward<ResponseType>(Response)));
			});
	}

private:
	HandlerType* Handler;
	FFuncType Func;
};
