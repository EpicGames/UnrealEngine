// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageContext.h"
#include "MessageEndpoint.h"

class IConcertLocalEndpoint;

/** Types of Concert Messages */
enum class EConcertMessageType
{
	/** Message is a event */
	Event,
	/** Message is a request */
	Request,
	/** Message is a response */
	Response,
};

/** States of Concert Messages */
enum class EConcertMessageState
{
	/** Message has been sent and is awaiting a response */
	Pending,
	/** Message has received its acknowledgment response, but is not yet complete */
	Acknowledged,
	/** Message has received its acknowledgment response and was completed */
	Completed,
	/** Message timed out */
	TimedOut,
};

/** Abstract for Concert Message */
class IConcertMessage
{
public:
	virtual ~IConcertMessage() = default;
	
	/** Get the ID of the message */
	virtual FGuid GetMessageId() const = 0;

	/** Get the order index of the message (for ordering reliable messages, used when ChannelId != UnreliableChannelId) */
	virtual uint16 GetMessageOrderIndex() const = 0;

	/** Get the ID of the channel this message was sent from */
	virtual uint16 GetChannelId() const = 0;

	/** Get is the message is flagged a reliable */
	virtual bool IsReliable() const = 0;

	/** Get the message type */
	virtual EConcertMessageType GetType() const = 0;

	/** Get the message state */
	virtual EConcertMessageState GetState() const = 0;

	/** Get the Concert Endpoint Id of the sender */
	virtual const FGuid& GetSenderId() const = 0;

	/** Get the Creation time of the message */
	virtual FDateTime GetCreationDate() const = 0;

	/** Construct this message data, allocates memory */
	virtual void* ConstructMessage() const = 0;

	/** Get the message type of the data allocated with ConstructMessage */
	virtual UScriptStruct* GetMessageType() const = 0;

	/** Get the template of the message that will be created with ConstructMessage */
	virtual const void* GetMessageTemplate() const = 0;

	/** Acknowledge the message */
	virtual void Acknowledge(const FConcertMessageContext& ConcertContext) = 0;

	/** Timeout the message */
	virtual void TimeOut() = 0;

	/** Annotations associated with this message. */
	virtual void SetAnnotations(TMap<FName, FString> Annotations) = 0;

	/** Annotations for sending across message bus */
	virtual const TMap<FName, FString>& GetAnnotations() const = 0;

protected:
	friend class IConcertLocalEndpoint;
	friend class IConcertRemoteEndpoint;

	/** Set the ID of the message. Call before ConstructMessage. */
	virtual void SetMessageId(const FGuid& MessageId) = 0;

	/** Set the order index of the message (for ordering reliable messages, used when ChannelId != UnreliableChannelId). Call before ConstructMessage. */
	virtual void SetOrderIndex(uint16 OrderIndex) = 0;

	/** Set the ID of the channel this message was sent from. Call before ConstructMessage. */
	virtual void SetChannelId(uint16 Channel) = 0;

	/** Set the Concert Endpoint Id of the sender. Call before ConstructMessage. */
	virtual void SetSenderId(const FGuid& SenderId) = 0;
};

/** Abstract class for Concert Event */
class IConcertEvent : public IConcertMessage
{
public:
	virtual ~IConcertEvent() = default;

	virtual EConcertMessageType GetType() const override
	{
		return EConcertMessageType::Event;
	}
};

/** Abstract class for Concert Request */
class IConcertRequest : public IConcertMessage
{
public:
	virtual ~IConcertRequest() = default;

	virtual EConcertMessageType GetType() const override
	{
		return EConcertMessageType::Request;
	}
};

/** Abstract class for Concert Response */
class IConcertResponse : public IConcertMessage
{
public:
	virtual ~IConcertResponse() = default;

	virtual EConcertMessageType GetType() const override
	{
		return EConcertMessageType::Response;
	}

protected:
	friend class IConcertLocalEndpoint;

	/** Set the ID of the request this response is for */
	virtual void SetRequestMessageId(const FGuid& RequestMessageId) = 0;
};

//////////////////////////////////////////////////////////////////////////

/** Implements common portion of Concert Message */
template<typename MessageBase, typename MessageType>
class TConcertMessage : public MessageBase
{
	static_assert(TIsDerivedFrom<MessageType, FConcertMessageData>::IsDerived, "TConcertMessage can only be constructed from struct deriving from FConcertMessageData.");
public:
	TConcertMessage(const MessageType& Message)
		: CreationDate(FDateTime::UtcNow())
		, MessageState(EConcertMessageState::Pending)
		, MessageTemplate(Message)
	{
	}

	TConcertMessage(MessageType&& Message)
		: CreationDate(FDateTime::UtcNow())
		, MessageState(EConcertMessageState::Pending)
		, MessageTemplate(MoveTemp(Message))
	{
	}

	virtual ~TConcertMessage() = default;

	virtual FGuid GetMessageId() const
	{
		return MessageTemplate.MessageId;
	}

	virtual uint16 GetMessageOrderIndex() const
	{
		return MessageTemplate.MessageOrderIndex;
	}

	virtual uint16 GetChannelId() const
	{
		return MessageTemplate.ChannelId;
	}

	virtual bool IsReliable() const override
	{
		return MessageTemplate.IsReliable();
	}

	virtual EConcertMessageState GetState() const override
	{
		return MessageState;
	}

	virtual const FGuid& GetSenderId() const override
	{
		return MessageTemplate.ConcertEndpointId;
	}

	virtual FDateTime GetCreationDate() const override
	{
		return CreationDate;
	}
	
	virtual void* ConstructMessage() const override
	{
		return FMessageEndpoint::MakeMessage<MessageType>(MessageTemplate);
	}

	virtual UScriptStruct* GetMessageType() const override
	{
		return MessageType::StaticStruct();
	}

	virtual const void* GetMessageTemplate() const override
	{
		return &MessageTemplate;
	}

	virtual void Acknowledge(const FConcertMessageContext& ConcertContext) override
	{
		MessageState = EConcertMessageState::Completed;
	}

	virtual void TimeOut() override
	{
		MessageState = EConcertMessageState::TimedOut;
	}

	virtual void SetAnnotations(TMap<FName, FString> InAnnotations) override
	{
		Annotations = MoveTemp(InAnnotations);
	}

	virtual const TMap<FName, FString>& GetAnnotations() const override
	{
		return Annotations;
	}
protected:
	virtual void SetMessageId(const FGuid& MessageId) override
	{
		MessageTemplate.MessageId = MessageId;
	}

	virtual void SetOrderIndex(uint16 OrderIndex) override
	{
		MessageTemplate.MessageOrderIndex = OrderIndex;
	}

	virtual void SetChannelId(uint16 Channel) override
	{
		MessageTemplate.ChannelId = Channel;
	}

	virtual void SetSenderId(const FGuid& SenderId) override
	{
		MessageTemplate.ConcertEndpointId = SenderId;
	}

	TMap<FName, FString> Annotations;
	FDateTime CreationDate;
	EConcertMessageState MessageState;
	MessageType MessageTemplate;
};

template<typename EventType>
class TConcertEvent : public TConcertMessage<IConcertEvent, EventType>
{
	static_assert(TIsDerivedFrom<EventType, FConcertEventData>::IsDerived, "TConcertEvent EventType must derive from FConcertEventData.");
public:
	TConcertEvent(const EventType& Event)
		: TConcertMessage<IConcertEvent, EventType>(Event)
	{}

	TConcertEvent(EventType&& Event)
		: TConcertMessage<IConcertEvent, EventType>(MoveTemp(Event))
	{}
	
	virtual ~TConcertEvent() = default;
};

template<typename RequestType, typename ResponseType>
class TConcertRequest : public TConcertMessage<IConcertRequest, RequestType>
{
	static_assert(TIsDerivedFrom<RequestType, FConcertRequestData>::IsDerived, "TConcertRequest RequestType must derive from FConcertRequestData.");
	static_assert(TIsDerivedFrom<ResponseType, FConcertResponseData>::IsDerived, "TConcertRequest ResponseType must derive from FConcertResponseData.");
public:
	TConcertRequest(const RequestType& Request)
		: TConcertMessage<IConcertRequest, RequestType>(Request)
	{}

	TConcertRequest(RequestType&& Request)
		: TConcertMessage<IConcertRequest, RequestType>(MoveTemp(Request))
	{}

	virtual ~TConcertRequest() = default;

	TFuture<ResponseType> GetFuture()
	{
		return Promise.GetFuture();
	}

	virtual void Acknowledge(const FConcertMessageContext& ConcertContext) override
	{
		check(ConcertContext.Message != nullptr);
		if (ConcertContext.MessageType->IsChildOf(FConcertAckData::StaticStruct()))
		{
			// Acknowledge the request to prevent resends, but don't complete it yet
			this->MessageState = EConcertMessageState::Acknowledged;
			return;
		}
		
		this->MessageState = EConcertMessageState::Completed;

		if (ConcertContext.MessageType == ResponseType::StaticStruct())
		{
			ResponseType* Response = (ResponseType*)ConcertContext.Message;
			Promise.SetValue(*Response);
		}
		else if (ConcertContext.MessageType->IsChildOf(FConcertResponseData::StaticStruct()))
		{
			// Received a generic response, just forward code and reason
			FConcertResponseData* GenericResponse = (FConcertResponseData*)ConcertContext.Message;
			ResponseType Response;
			Response.ResponseCode =  GenericResponse->ResponseCode;
			Response.Reason = GenericResponse->Reason;
			Promise.SetValue(MoveTemp(Response));
		}
		else
		{
			// Set the response code to invalid before sending result
			ResponseType Response;
			Response.ResponseCode = EConcertResponseCode::InvalidRequest;
			Promise.SetValue(MoveTemp(Response));
		}
	}

	virtual void TimeOut() override
	{
		if (this->MessageState != EConcertMessageState::Completed)
		{
			this->MessageState = EConcertMessageState::TimedOut;

			ResponseType Response;
			Response.ResponseCode = EConcertResponseCode::TimedOut;
			Promise.SetValue(Response);
		}
	}

private:
	TPromise<ResponseType> Promise;
};

template<typename ResponseType>
class TConcertResponse : public TConcertMessage<IConcertResponse, ResponseType>
{
	static_assert(TIsDerivedFrom<ResponseType, FConcertResponseData>::IsDerived, "TConcertResponse ResponseType must derive from FConcertResponseData.");
public: 
	TConcertResponse(const ResponseType& Response)
		: TConcertMessage<IConcertResponse, ResponseType>(Response)
	{
		ValidateResponseCode();
	}

	TConcertResponse(ResponseType&& Response)
		: TConcertMessage<IConcertResponse, ResponseType>(MoveTemp(Response))
	{
		ValidateResponseCode();
	}

	virtual ~TConcertResponse() = default;

protected:
	virtual void SetRequestMessageId(const FGuid& RequestMessageId) override
	{
		this->MessageTemplate.RequestMessageId = RequestMessageId;
	}

private:
	// Set the response code to successful if the response generator left it pending
	void ValidateResponseCode()
	{
		this->MessageTemplate.ResponseCode = this->MessageTemplate.ResponseCode == EConcertResponseCode::Pending ? EConcertResponseCode::Success : this->MessageTemplate.ResponseCode;
	}
};
