// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertTransportLoggerPtr.h"

class IConcertMessage;
struct FConcertMessageContext;

struct FConcertAckData;
struct FConcertEndpointClosedData;
struct FConcertReliableHandshakeData;

/**
 * Logging interface for Concert Transport layer
 */
class IConcertTransportLogger
{
public:
	enum class EMessageDiscardedReason : uint8
	{
		NotRequired,
		AlreadyProcessed,
		UnknownEndpoint,
	};

	/** Virtual destructor */
	virtual ~IConcertTransportLogger() = default;

	/** Is this log currently logging? */
	virtual bool IsLogging() const = 0;

	/** Start logging */
	virtual void StartLogging() = 0;

	/** Stop logging */
	virtual void StopLogging() = 0;

	/** Flush the log, processing any pending entries */
	virtual void FlushLog() = 0;

	/** Log a timeout for a Message sent to EndpointId */
	virtual void LogTimeOut(const TSharedRef<IConcertMessage>& Message, const FGuid& EndpointId, const FDateTime& UtcNow) = 0;

	/** Log an acknowledgment sent to the destination endpoint */
	virtual void LogSendAck(const FConcertAckData& AckData, const FGuid& DestEndpoint) = 0;

	/** Log an endpoint being closed on the remote peer (us!) */
	virtual void LogSendEndpointClosed(const FConcertEndpointClosedData& EndpointClosedData, const FGuid& DestEndpoint, const FDateTime& UtcNow) = 0;

	/** Log a reliable handshake sent to the destination endpoint */
	virtual void LogSendReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow) = 0;

	/** Log a reliable handshake received from the source endpoint */
	virtual void LogReceiveReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow) = 0;

	/** Log the publication of Message */
	virtual void LogPublish(const TSharedRef<IConcertMessage>& Message) = 0;

	/** Log the sending of Message to DestEndpoint */
	virtual void LogSend(const TSharedRef<IConcertMessage>& Message, const FGuid& DestEndpoint) = 0;

	/** Log a message received from this ConcertContext */
	virtual void LogMessageReceived(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a message queued from this ConcertContext */
	virtual void LogMessageQueued(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a message discarded from this ConcertContext */
	virtual void LogMessageDiscarded(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint, const EMessageDiscardedReason Reason) = 0;

	/** Log an event processed from this ConcertContext */
	virtual void LogProcessEvent(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a request processed from this ConcertContext */
	virtual void LogProcessRequest(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a response processed from this ConcertContext */
	virtual void LogProcessResponse(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a acknowledgment processed from this ConcertContext */
	virtual void LogProcessAck(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log the discovery of a remote endpoint with EndpointId*/
	virtual void LogRemoteEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) = 0;

	/** Log a remote endpoint with EndpointId being considered stale or timed out */
	virtual void LogRemoteEndpointTimeOut(const FGuid& EndpointId, const FDateTime& UtcNow) = 0;

	/** Log a remote endpoint with EndpointId being as being closed */
	virtual void LogRemoteEndpointClosure(const FGuid& EndpointId, const FDateTime& UtcNow) = 0;
};

/**
 * Wrapper class around a IConcertTransportLogger pointer
 */
class FConcertTransportLoggerWrapper
{
public:
	FConcertTransportLoggerWrapper(const IConcertTransportLoggerPtr& InLogger)
		: Logger(InLogger)
	{
	}

	IConcertTransportLoggerPtr GetLogger() const
	{
		return Logger;
	}

	bool IsLogging() const
	{
		return Logger.IsValid() && Logger->IsLogging();
	}

	void StartLogging() const
	{
		if (Logger.IsValid())
		{
			Logger->StartLogging();
		}
	}

	void StopLogging() const
	{
		if (Logger.IsValid())
		{
			Logger->StopLogging();
		}
	}

	void FlushLog() const
	{
		if (Logger.IsValid())
		{
			Logger->FlushLog();
		}
	}

	void LogTimeOut(const TSharedRef<IConcertMessage>& Message, const FGuid& EndpointId, const FDateTime& UtcNow) const
	{
		if (Logger.IsValid())
		{
			Logger->LogTimeOut(Message, EndpointId, UtcNow);
		}
	}

	void LogSendAck(const FConcertAckData& AckData, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogSendAck(AckData, DestEndpoint);
		}
	}

	void LogSendEndpointClosed(const FConcertEndpointClosedData& EndpointClosedData, const FGuid& DestEndpoint, const FDateTime& UtcNow) const
	{
		if (Logger.IsValid())
		{
			Logger->LogSendEndpointClosed(EndpointClosedData, DestEndpoint, UtcNow);
		}
	}

	void LogSendReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& DestEndpoint, const FDateTime& UtcNow)
	{
		if (Logger.IsValid())
		{
			Logger->LogSendReliableHandshake(ReliableHandshakeData, DestEndpoint, UtcNow);
		}
	}

	void LogReceiveReliableHandshake(const FConcertReliableHandshakeData& ReliableHandshakeData, const FGuid& SrcEndpoint, const FDateTime& UtcNow)
	{
		if (Logger.IsValid())
		{
			Logger->LogReceiveReliableHandshake(ReliableHandshakeData, SrcEndpoint, UtcNow);
		}
	}

	void LogPublish(const TSharedRef<IConcertMessage>& Message) const
	{
		if (Logger.IsValid())
		{
			Logger->LogPublish(Message);
		}
	}

	void LogSend(const TSharedRef<IConcertMessage>& Message, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogSend(Message, DestEndpoint);
		}
	}

	void LogMessageReceived(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogMessageReceived(ConcertContext, DestEndpoint);
		}
	}

	void LogMessageQueued(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogMessageQueued(ConcertContext, DestEndpoint);
		}
	}

	void LogMessageDiscarded(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint, const IConcertTransportLogger::EMessageDiscardedReason Reason) const
	{
		if (Logger.IsValid())
		{
			Logger->LogMessageDiscarded(ConcertContext, DestEndpoint, Reason);
		}
	}

	void LogProcessEvent(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogProcessEvent(ConcertContext, DestEndpoint);
		}
	}

	void LogProcessRequest(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogProcessRequest(ConcertContext, DestEndpoint);
		}
	}

	void LogProcessResponse(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogProcessResponse(ConcertContext, DestEndpoint);
		}
	}

	void LogProcessAck(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogProcessAck(ConcertContext, DestEndpoint);
		}
	}

	void LogRemoteEndpointDiscovery(const FConcertMessageContext& ConcertContext, const FGuid& DestEndpoint) const
	{
		if (Logger.IsValid())
		{
			Logger->LogRemoteEndpointDiscovery(ConcertContext, DestEndpoint);
		}
	}

	void LogRemoteEndpointTimeOut(const FGuid& EndpointId, const FDateTime& UtcNow) const
	{
		if (Logger.IsValid())
		{
			Logger->LogRemoteEndpointTimeOut(EndpointId, UtcNow);
		}
	}

	void LogRemoteEndpointClosure(const FGuid& EndpointId, const FDateTime& UtcNow) const
	{
		if (Logger.IsValid())
		{
			Logger->LogRemoteEndpointClosure(EndpointId, UtcNow);
		}
	}

	void Reset()
	{
		Logger.Reset();
	}

private:
	IConcertTransportLoggerPtr Logger;
};
