// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IConcertTransportLogger;

struct FConcertEndpointContext;

typedef TSharedRef<IConcertTransportLogger, ESPMode::ThreadSafe> IConcertTransportLoggerRef;
typedef TSharedPtr<IConcertTransportLogger, ESPMode::ThreadSafe> IConcertTransportLoggerPtr;

/**
 * Factory function used to create a concrete instance of IConcertTransportLogger.
 */
typedef TFunction<IConcertTransportLoggerRef(const FConcertEndpointContext&)> FConcertTransportLoggerFactory;
