// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertIdentifierTableData.h"

#define UE_API CONCERTTRANSPORT_API

/** Key functions that compare FName instances by only their display string (to perform a case-sensitive plain string comparison) */
template <typename ValueType>
struct TConcertIdentifierTable_CaseSensitivePlainNameKeyFuncs : BaseKeyFuncs<ValueType, FName, /*bInAllowDuplicateKeys*/false>
{
	static FORCEINLINE const FName& GetSetKey(const TPair<FName, ValueType>& Element)
	{
		return Element.Key;
	}
	static FORCEINLINE bool Matches(const FName& A, const FName& B)
	{
		return A.GetDisplayIndex() == B.GetDisplayIndex();
	}
	static FORCEINLINE uint32 GetKeyHash(const FName& Key)
	{
		return GetTypeHash(Key.GetDisplayIndex());
	}
};

/** Cache of identifiers (currently names) that have been serialized locally and should be sent along with the serialized data */
class FConcertLocalIdentifierTable
{
public:
	FConcertLocalIdentifierTable() = default;
	UE_API explicit FConcertLocalIdentifierTable(const FConcertLocalIdentifierState& InState);

	/**
	 * Constant used to force names to serialize as their FName table index value (non-portable!)
	 */
	static UE_API FConcertLocalIdentifierTable* ForceFNameTableIndex;

	/**
	 * Does this table have any mappings in it?
	 */
	UE_API bool HasMappings() const;

	/**
	 * Map the given name to its identifier index.
	 */
	UE_API int32 MapName(const FName& InName);

	/**
	 * Unmap the given name from its identifier index (or NAME_None).
	 * @return true if the name was unmapped, false if the name wasn't mapped (OutName will be set to NAME_None).
	 */
	UE_API bool UnmapName(const int32 InIndex, FName& OutName) const;

	/**
	 * Is the given identifier index mapped?
	 */
	UE_API bool HasName(const int32 InIndex) const;

	/**
	 * Is the given name mapped?
	 */
	UE_API bool HasName(const FName& InName, int32* OutIndexPtr = nullptr) const;

	/**
	 * Set the state of this identifier table.
	 */
	UE_API void SetState(const FConcertLocalIdentifierState& InState);

	/**
	 * Get the state of this identifier table.
	 */
	UE_API void GetState(FConcertLocalIdentifierState& OutState) const;

private:
	/** Array of locally mapped names */
	TArray<FName> MappedNames;
	/** Map of known names to their index in MappedNames (name -> index) */
	TMap<FName, int32, FDefaultSetAllocator, TConcertIdentifierTable_CaseSensitivePlainNameKeyFuncs<int32>> NameToMappedIndex;
};

#undef UE_API
