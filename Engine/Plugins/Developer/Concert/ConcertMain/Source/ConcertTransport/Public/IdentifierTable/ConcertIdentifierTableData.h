// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertIdentifierTableData.generated.h"

USTRUCT()
struct FConcertLocalIdentifierState
{
	GENERATED_BODY()

	void Reset(const int32 NewSize = 0)
	{
		MappedNames.Reset(NewSize);
	}

	void Empty(const int32 Slack = 0)
	{
		MappedNames.Empty(Slack);
	}

	bool IsEmpty() const
	{
		return MappedNames.Num() == 0;
	}

	UPROPERTY()
	TArray<FString> MappedNames;
};
