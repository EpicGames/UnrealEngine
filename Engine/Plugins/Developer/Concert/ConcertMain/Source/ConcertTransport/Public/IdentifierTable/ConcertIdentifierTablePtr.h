// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FConcertLocalIdentifierTable;

typedef TSharedRef<FConcertLocalIdentifierTable, ESPMode::ThreadSafe> FConcertLocalIdentifierTableRef;
typedef TSharedPtr<FConcertLocalIdentifierTable, ESPMode::ThreadSafe> FConcertLocalIdentifierTablePtr;
typedef TSharedRef<const FConcertLocalIdentifierTable, ESPMode::ThreadSafe> FConcertLocalIdentifierTableConstRef;
typedef TSharedPtr<const FConcertLocalIdentifierTable, ESPMode::ThreadSafe> FConcertLocalIdentifierTableConstPtr;
