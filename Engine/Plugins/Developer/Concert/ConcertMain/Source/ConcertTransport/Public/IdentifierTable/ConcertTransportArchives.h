// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#define UE_API CONCERTTRANSPORT_API

class FConcertLocalIdentifierTable;

/** Archive for writing identifiers (currently names) in a way that avoids duplication by caching them against their internal key, which can then be mapped over the network */
class FConcertIdentifierWriter : public FMemoryWriter
{
public:
	UE_API FConcertIdentifierWriter(FConcertLocalIdentifierTable* InLocalIdentifierTable, TArray<uint8>& InBytes, bool bIsPersistent = false);

	using FMemoryWriter::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(FName& Name) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

protected:
	FConcertLocalIdentifierTable* GetLocalIdentifierTable() const { return LocalIdentifierTable; }

private:
	FConcertLocalIdentifierTable* LocalIdentifierTable;
};

/** Archive for reading identifiers (currently names) in a way that avoids duplication by caching them against their internal key, which can then be mapped over the network */
class FConcertIdentifierReader : public FMemoryReader
{
public:
	UE_API FConcertIdentifierReader(const FConcertLocalIdentifierTable* InLocalIdentifierTable, const TArray<uint8>& InBytes, bool bIsPersistent = false);

	using FMemoryReader::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(FName& Name) override;
	UE_API FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

protected:
	const FConcertLocalIdentifierTable* GetLocalIdentifierTable() const { return LocalIdentifierTable; }

private:
	const FConcertLocalIdentifierTable* LocalIdentifierTable;
};

/** Archive for rewriting identifiers (currently names) so that they belong to a different identifier table */
class FConcertIdentifierRewriter : public FMemoryArchive
{
public:
	UE_API FConcertIdentifierRewriter(const FConcertLocalIdentifierTable* InLocalIdentifierTable, FConcertLocalIdentifierTable* InRewriteIdentifierTable, TArray<uint8>& InBytes, bool bIsPersistent = false);

	using FMemoryArchive::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	UE_API virtual FArchive& operator<<(FName& Name) override;
	UE_API virtual FArchive& operator<<(FSoftObjectPath& AssetPtr) override;
	UE_API virtual int64 TotalSize() override final;
	UE_API virtual void Serialize(void* Data, int64 Num) override final;
	UE_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

protected:
	const FConcertLocalIdentifierTable* GetLocalIdentifierTable() const { return LocalIdentifierTable; }
	FConcertLocalIdentifierTable* GetRewriteIdentifierTable() const { return RewriteIdentifierTable; }

	template <typename T>
	void RewriteData(const int64 OldDataOffset, const int64 OldDataSize, T& NewData)
	{
		// Write the name to the scratch buffer using the rewrite identifier table
		{
			ScratchBytes.Reset();
			FConcertIdentifierWriter ScratchWriter(RewriteIdentifierTable, ScratchBytes, IsPersistent());
			checkf(ScratchBytes.Num() == 0, TEXT("FConcertIdentifierWriter wrote extra data!"));
			ScratchWriter << NewData;
		}

		// Patch the scratch buffer into the real buffer
		// The data must be the same size to avoid serialization issues with tagged data
		{
			const int64 NewDataSize = ScratchBytes.Num();
			if (ensureAlwaysMsgf(OldDataSize == NewDataSize, TEXT("FConcertIdentifierRewriter: Rewritten data must be the same size to avoid issues with tagged property serialization! (old: %" INT64_FMT ", new: %" INT64_FMT ")"), OldDataSize, NewDataSize))
			{
				FMemory::Memcpy(Bytes.GetData() + OldDataOffset, ScratchBytes.GetData(), NewDataSize);
			}
			else
			{
				SetError();
			}
		}
	}

private:
	const FConcertLocalIdentifierTable* LocalIdentifierTable = nullptr;
	FConcertLocalIdentifierTable* RewriteIdentifierTable = nullptr;
	TArray<uint8>& Bytes;
	TArray<uint8> ScratchBytes;
};

#undef UE_API
