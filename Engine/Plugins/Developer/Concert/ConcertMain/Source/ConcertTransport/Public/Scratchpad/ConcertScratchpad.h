// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CONCERTTRANSPORT_API

/** Scratchpad of id -> value pairs corresponding to a particular client */
class FConcertScratchpad
{
private:
	/** Scratchpad value interface */
	struct IScratchpadValue
	{
		virtual ~IScratchpadValue() = default;
		virtual void* GetValuePtr() = 0;
		virtual const void* GetValuePtr() const = 0;
	};
	typedef TSharedPtr<IScratchpadValue, ESPMode::ThreadSafe> IScratchpadValuePtr;

	/** Scratchpad value implementation */
	template <typename T>
	struct TScratchpadValue : public IScratchpadValue
	{
		explicit TScratchpadValue(T&& InValue)
			: Value(Forward<T>(InValue))
		{
		}

		virtual void* GetValuePtr() override
		{
			return &Value;
		}

		virtual const void* GetValuePtr() const override
		{
			return &Value;
		}

		T Value;
	};

public:
	/**
	 * Does the scratchpad have a value for the given key?
	 */
	UE_API bool HasValue(const FName& InId) const;

	/**
	 * Set the scratchpad value associated with the given key.
	 */
	template <typename T>
	void SetValue(const FName& InId, const T& InValue)
	{
		InternalSetValue(InId, MakeShared<TScratchpadValue<T>, ESPMode::ThreadSafe>(CopyTemp(InValue)));
	}

	/**
	 * Set the scratchpad value associated with the given key.
	 */
	template <typename T>
	void SetValue(const FName& InId, T&& InValue)
	{
		InternalSetValue(InId, MakeShared<TScratchpadValue<T>, ESPMode::ThreadSafe>(Forward<T>(InValue)));
	}

	/**
	 * Get the scratchpad value associated with the given key (if any).
	 */
	template <typename T>
	T* GetValue(const FName& InId)
	{
		IScratchpadValuePtr ScratchpadValue = InternalGetValue(InId);
		return ScratchpadValue.IsValid() ? static_cast<T*>(ScratchpadValue->GetValuePtr()) : nullptr;
	}

	/**
	 * Get the scratchpad value associated with the given key (if any).
	 */
	template <typename T>
	const T* GetValue(const FName& InId) const
	{
		IScratchpadValuePtr ScratchpadValue = InternalGetValue(InId);
		return ScratchpadValue.IsValid() ? static_cast<T*>(ScratchpadValue->GetValuePtr()) : nullptr;
	}

	/**
	 * Get the scratchpad value associated with the given key, or assert if missing.
	 */
	template <typename T>
	T& GetValueChecked(const FName& InId)
	{
		T* ValuePtr = GetValue<T>(InId);
		check(ValuePtr);
		return *ValuePtr;
	}

	/**
	 * Get the scratchpad value associated with the given key, or assert if missing.
	 */
	template <typename T>
	const T& GetValueChecked(const FName& InId) const
	{
		const T* ValuePtr = GetValue<T>(InId);
		check(ValuePtr);
		return *ValuePtr;
	}

private:
	/**
	 * Set the internal scratchpad value associated with the given key.
	 */
	UE_API void InternalSetValue(const FName& InId, IScratchpadValuePtr&& InValue);

	/**
	 * Get the internal scratchpad value associated with the given key (if any).
	 */
	UE_API IScratchpadValuePtr InternalGetValue(const FName& InId) const;

	/** Critical section protecting concurrent access to ScratchpadValues */
	mutable FCriticalSection ScratchpadValuesCS;
	/** Map of values (id -> value) */
	TMap<FName, IScratchpadValuePtr> ScratchpadValues;
};

#undef UE_API
