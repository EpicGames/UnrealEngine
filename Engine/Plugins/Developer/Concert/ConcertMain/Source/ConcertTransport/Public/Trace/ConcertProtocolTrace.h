// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/ConcertTraceConfig.h"
#include "Trace/Private/ConcertProtocolTraceInternal.h"

/*
 * This file defines the Concert Protocol Tracing macros.
 * Unreal Insights does not understand it by default and we add extensions in the ConcertInsights plugin to visualize it.
 * 
 * The heart of Concert Protocol Tracing are sequences.
 * A sequence consists of actions that are performed by multiple endpoints.
 * Example of a sequence:
 *  - Every frame, client 1 collects data to replicate and sends it to the server.
 *  - The server enqueues the data and sends it to client 2.
 *  - Client 2 applies the data.
 * Sequences are associated with a EProtocolSuite and grouped accordingly by the ConcertInsights extension.
 *
 * For now there is only one type of trace: traced objects. See CONCERT_TRACE_OBJECT_ Macros.
 * These associate actions with an object and a sequence ID, which groups together related changes.
 * For example, in the context of replication a single frame's data being sent across the clients gets a single sequence ID:
 *	- client 1 could trace a. how long it takes to serialize the data and then b. how long to compress it.
 *	- the server could trace how long it takes to process the data
 *	- client 2 could trace a. how long it apply the data to the UObject
 *	
 * ConcertInsights visualizes networking transport times.
 * Transmission starts with CONCERT_TRACE_OBJECT_TRANSMISSION_START and ends with CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE.
 * 
 * TODO DP: We might want to add CONCERT_TRACE_OBJECT_SEND and CONCERT_TRACE_OBJECT_RECEIVE macros to be explicit about sending and receiving.
 * 
 * Every endpoint writes to its own .utrace file and must be in context of a session ID (i.e. clients are in a session, server is told which session is being traced).
 * When beginning to trace, the first event sent to the .utrace files is CONCERT_TRACE_INIT, which sets data needed to aggregate the .utrace files, such as endpoint IDs.
 * Finally, ConcertInsights aggregates the .utrace files in the UI. The related .utrace files are grouped by SessionID, which Unreal Insights saves
 * in the file by reading the -tracesessionguid command line; CONCERT_TRACE_INIT associates the .utrace file with the client / server endpoint ID.
 * ConcertInsightsSynchronizedTrace implements synchronized recording.
 */

namespace UE::ConcertTrace
{
	enum class EProtocolSuite : uint8
	{
		Unknown,

		/** The traces are related to the replication system */
		Replication
	};

	FORCEINLINE uint8 ProtocolSuiteToInt(EProtocolSuite Id) { return static_cast<uint8>(Id); }

	/** @return Whether the given protocol is currently being traced */
	CONCERTTRANSPORT_API bool IsTracing(EProtocolSuite Id);
	/** @return Whether replication is currently being traced. */
	FORCEINLINE bool IsTracingReplication() { return IsTracing(EProtocolSuite::Replication); }
}

#if UE_CONCERT_TRACE_ENABLED

/**
 * Event that is emitted when a trace is started in the local instance.
 * 
 * @param InEndpointIdAsTOptional TOptional<FGuid> that identifies the local instance endpoint ID in the active session.
 *	This should always be set if started via ConcertInsightsSynchronizedTrace.
 *	May be unset when the trace is conventionally started by user.
 * @param IsServer Whether the local instance is the server
 */
#define CONCERT_TRACE_INIT(EndpointIdAsTOptional, InEndpointDisplayFString, IsServer) INTERNAL_CONCERT_TRACE_INIT(EndpointIdAsTOptional, InEndpointDisplayFString, IsServer)

/********** Generics Macros - Create new macros that wrap these when you add a new traced protocol **********/

/**
 * Logs an event with a start an end time.
 * In Insights, this will show a track that begins at the current time and ends when the scope terminates.
 * Supports nesting.
 * 
 * @param EventName The name of the event to show
 * @param ObjectPath The object this event pertains to. Events with the same ObjectPath and SequenceID are grouped in one track.
 * @param SequenceId Which change to associate this event to (use-case defined). Events with the same ObjectPath and SequenceID are grouped in one track.
 * @param ProtocolId The protocol to group this event under.
 */
#define CONCERT_TRACE_OBJECT_SCOPE(EventName, ObjectPath, SequenceId, ProtocolId) INTERNAL_CONCERT_TRACE_OBJECT_SCOPE(EventName, ObjectPath, SequenceId, ProtocolId)

/** Traces that the local endpoint has handed the object's sequence data to message bus. The time from now until CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE is visualized as transit time. */
#define CONCERT_TRACE_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId, ProtocolId) INTERNAL_CONCERT_TRACE_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId, ProtocolId)
/** Traces that the local endpoint has received the object's sequence data. The time from now the previous CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE until now is visualized as transit time. */
#define CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId, ProtocolId) INTERNAL_CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId, ProtocolId)
/** The object's data has been fully processed. End of the object track. */
#define CONCERT_TRACE_OBJECT_SINK(SinkName, ObjectPath, SequenceId, ProtocolId) INTERNAL_CONCERT_TRACE_OBJECT_SINK(SinkName, ObjectPath, SequenceId, ProtocolId)

/********** Replication Protocol **********/

// TODO DP: Add CONCERT_TRACE_REPLICATION_OBJECT_ANONYMOUS_CONTEXT_SCOPE(EventName) which could be used to give more context of what other actions are being performed.
// Example: This could be added at the root of Tick() so we know what other parts of Concert are delaying packet delivery.
// Alternative could also just be using CONCERT_TRACE_SCOPE ...

/**
 * Logs an event with a start an end time.
 * In Insights, this will show a track that begins at the current time and ends at the start time of the next event.
 * Supports nesting.
 * 
 * @param EventName The name of the event to show
 * @param ObjectPath The object this event pertains to.
 * @param SequenceId Determined by sending client and passed along to all other endpoints. 
 */
#define CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(EventName, ObjectPath, SequenceId) CONCERT_TRACE_OBJECT_SCOPE(EventName, ObjectPath, SequenceId, ProtocolSuiteToInt(UE::ConcertTrace::EProtocolSuite::Replication))

/** Traces that the local endpoint has handed the object's sequence data to message bus. The time from now until CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_RECEIVE is visualized as transit time. */
#define CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId) CONCERT_TRACE_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId, ProtocolSuiteToInt(UE::ConcertTrace::EProtocolSuite::Replication))
/** Traces that the local endpoint has received the object's sequence data. The time from now the previous CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_START until now is visualized as transit time. */
#define CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId) CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId, ProtocolSuiteToInt(UE::ConcertTrace::EProtocolSuite::Replication))
/** The object's data has been fully processed. End of the object track. */
#define CONCERT_TRACE_REPLICATION_OBJECT_SINK(SinkName, ObjectPath, SequenceId) CONCERT_TRACE_OBJECT_SINK(SinkName, ObjectPath, SequenceId, ProtocolSuiteToInt(UE::ConcertTrace::EProtocolSuite::Replication))

#else
#define CONCERT_TRACE_INIT(EndpointIdAsTOptional, InEndpointDisplayFString, IsServer)
#define CONCERT_TRACE_OBJECT_SCOPE(EventName, ObjectPath, SequenceId, ProtocolId)
#define CONCERT_TRACE_OBJECT_EVENT(EventName, ObjectPath, SequenceId, ProtocolId)
#define CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(EventName, ObjectPath, SequenceId)
#define CONCERT_TRACE_REPLICATION_OBJECT_EVENT(EventName, ObjectPath, SequenceId)
#define CONCERT_TRACE_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId, ProtocolId)
#define CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId, ProtocolId)
#define CONCERT_TRACE_OBJECT_SINK(ObjectPath, SequenceId, ProtocolId)
#define CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_START(DestinationEndpointId, ObjectPath, SequenceId)
#define CONCERT_TRACE_REPLICATION_OBJECT_TRANSMISSION_RECEIVE(ObjectPath, SequenceId)
#define CONCERT_TRACE_REPLICATION_OBJECT_SINK(SinkName, ObjectPath, SequenceId)
#endif