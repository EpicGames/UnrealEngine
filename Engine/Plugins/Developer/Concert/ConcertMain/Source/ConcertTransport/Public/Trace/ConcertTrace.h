// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTraceConfig.h"
#include "Trace/Trace.h"

/*
 * This file defines the common UE tracing utilities.
 * Common means the traces that Unreal Insights already understands without any extensions added by ConcertInsights plugin.
 */

#if UE_CONCERT_TRACE_ENABLED
	UE_TRACE_CHANNEL_EXTERN(ConcertChannel, CONCERTTRANSPORT_API);
#	define SCOPED_CONCERT_TRACE(TraceName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TraceName, ConcertChannel)
#else
#	define SCOPED_CONCERT_TRACE(TraceName)
#endif