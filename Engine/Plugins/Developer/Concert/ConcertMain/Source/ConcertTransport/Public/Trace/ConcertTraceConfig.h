// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Trace/Config.h"

#if !defined(UE_CONCERT_TRACE_ENABLED)
#	if UE_TRACE_ENABLED && CPUPROFILERTRACE_ENABLED && !UE_BUILD_SHIPPING
#		define UE_CONCERT_TRACE_ENABLED 1
#	else
#		define UE_CONCERT_TRACE_ENABLED 0
#	endif
#endif