// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/ConcertTraceConfig.h"

#if UE_CONCERT_TRACE_ENABLED

#include "Trace/Private/ConcertScopedObjectTrace.h"

namespace UE::ConcertTrace
{
	/** Sent as part of the Init event for backwards compatibility. */
	enum class EConcertTraceVersion : uint8
	{
		/** Initial version of Concert Insights. */
		Initial = 0,

		// Add new entries above
		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};
	
	CONCERTTRANSPORT_API void Init(const TOptional<FGuid>& LocalEndpointId, const FString& ClientDisplayName, bool bIsServer);
	CONCERTTRANSPORT_API void LogTransmissionStart(uint8 Protocol, uint32 SequenceId, const FSoftObjectPath& ObjectPath, const FGuid& TargetEndpointId);
	CONCERTTRANSPORT_API void LogTransmissionReceive(uint8 Protocol, uint32 SequenceId, const FSoftObjectPath& ObjectPath);
	CONCERTTRANSPORT_API void LogSink(uint8 Protocol, uint32 SequenceId, const ANSICHAR* SinkName, const FSoftObjectPath& ObjectPath);
}

#define INTERNAL_CONCERT_TRACE_INIT(InEndpointIdTOptional, InEndpointDisplayFString, InIsServer) UE::ConcertTrace::Init(InEndpointIdTOptional, InEndpointDisplayFString, InIsServer);

#define INTERNAL_CONCERT_TRACE_OBJECT_SCOPE(InEventName, InObjectPath, InSequenceId, InProtocolId) UE::ConcertTrace::FConcertScopedObjectTrace UE_JOIN(TheScope, __LINE__)(InProtocolId, InSequenceId, #InEventName, InObjectPath);

#define INTERNAL_CONCERT_TRACE_OBJECT_TRANSMISSION_START(InDestinationEndpointId, InObjectPath, InSequenceId, InProtocolId) UE::ConcertTrace::LogTransmissionStart(InProtocolId, InSequenceId, InObjectPath, InDestinationEndpointId);
#define INTERNAL_CONCERT_TRACE_OBJECT_TRANSMISSION_RECEIVE(InObjectPath, InSequenceId, InProtocolId) UE::ConcertTrace::LogTransmissionReceive(InProtocolId, InSequenceId, InObjectPath);
#define INTERNAL_CONCERT_TRACE_OBJECT_SINK(InSinkName, InObjectPath, InSequenceId, InProtocolId) UE::ConcertTrace::LogSink(InProtocolId, InSequenceId, #InSinkName, InObjectPath);

#endif