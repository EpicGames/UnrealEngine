// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertTrace::Private
{
	struct FConcertReplicationSequenceContext
	{
		FSoftObjectPath Object;
		int32 FrameNumber;

		FString ToString() const { return Object.ToString() + FString::FromInt(FrameNumber); }
	};
}