// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/ConcertTraceConfig.h"

#if UE_CONCERT_TRACE_ENABLED

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertTrace
{
	class FConcertScopedObjectTrace : public FNoncopyable
	{
		const bool bShouldTrace;
		
		const uint8 Protocol;
		const uint32 SequenceId;
		const ANSICHAR* EventName;
		const FSoftObjectPath ObjectPath;
		
	public:

		CONCERTTRANSPORT_API FConcertScopedObjectTrace(uint8 InProtocol, uint32 InSequenceId, const ANSICHAR* InEventName, FSoftObjectPath InObjectPath);
		CONCERTTRANSPORT_API ~FConcertScopedObjectTrace();
	};
}
#endif
